{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Datasmith Runtime",
	"Description": "",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DatasmithRuntime",
			"Type": "Runtime",
			"LoadingPhase": "PostDefault",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "DatasmithImporter",
			"Enabled": true
		},
		{
			"Name": "UdpMessaging",
			"Enabled": true
		}
	]
}