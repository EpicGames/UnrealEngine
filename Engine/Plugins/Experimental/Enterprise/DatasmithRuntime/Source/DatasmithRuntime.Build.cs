// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildTool;

public class DatasmithRuntime : ModuleRules
{
	public DatasmithRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"DatasmithContent",
				"DatasmithCore",
				"DatasmithNativeTranslator",
				"DatasmithTranslator",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CinematicCamera",
				"Core",
				"CoreUObject",
				"DirectLink",
				"Engine",
				"IESFile",
				"FreeImage",
				"Landscape",
				"LevelSequence",
				"MeshDescription",
				"MeshUtilitiesCommon",
				"RawMesh",
				"RHI",
				"PhysicsCore",
				"RenderCore",
				"SlateCore",
				"StaticMeshDescription",
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DesktopPlatform",
					"DeveloperToolSettings",
					"MessageLog",
                    "UnrealEd",
                }
			);
		}

		// Set environment variable DIRECTLINK_LOG to get DirectLink logging
		string DirectLog = System.Environment.GetEnvironmentVariable("DIRECTLINK_LOG");
		if (DirectLog != null)
		{
			PublicDefinitions.Add("DIRECTLINK_LOG");
		}
	}
}