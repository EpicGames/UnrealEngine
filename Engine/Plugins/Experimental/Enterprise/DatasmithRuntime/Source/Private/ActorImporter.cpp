// Copyright Epic Games, Inc. All Rights Reserved.


#include "Components/StaticMeshComponent.h"
#include "DatasmithRuntimeUtils.h"

#include "IDatasmithSceneElements.h"

#include "Algo/AnyOf.h"
#include "CineCameraActor.h"
#include "CineCameraComponent.h"
#include "RenderingThread.h"
#include "UObject/Package.h"

namespace DatasmithRuntime
{
	// Tag added to created scene components and actors
	// Used during reset process
	const FName RuntimeTag("Datasmith.Runtime.Tag");

	void DeleteSceneComponent(USceneComponent* SceneComponent);

	// Recursively destroy actor from world. Children are deleted first.
	void DeleteActorRecursive(UWorld* GameWorld, AActor* Actor);

	// Set scene component's visibility to false and remove geometry from static mesh component
	// Call before deletion to ensure no asset is used
	void HideSceneComponent(USceneComponent* SceneComponent);

	void FSceneImporter::ProcessActorElement(const TSharedPtr< IDatasmithActorElement >& ActorElement, FSceneGraphId ParentId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::ProcessActorElement);

		FActorData& ActorData = FindOrAddActorData(ActorElement);

		if (ActorData.HasState(EAssetState::Processed))
		{
			return;
		}

		ActorData.ParentId = ParentId;
		ActorData.WorldTransform = FTransform(ActorElement->GetRotation(), ActorElement->GetTranslation(), ActorElement->GetScale()) * RootComponent->GetComponentTransform();

		bool bProcessSuccessful = false;

		if (ActorElement->IsA(EDatasmithElementType::StaticMeshActor))
		{
			IDatasmithMeshActorElement* MeshActorElement = static_cast<IDatasmithMeshActorElement*>(ActorElement.Get());

			ActorData.Type = EDataType::MeshActor;
			bProcessSuccessful = ProcessMeshActorData(ActorData, MeshActorElement);
		}
		else if (ActorElement->IsA(EDatasmithElementType::Light))
		{
			IDatasmithLightActorElement* LightActorElement = static_cast<IDatasmithLightActorElement*>(ActorElement.Get());

			ActorData.Type = EDataType::LightActor;
			bProcessSuccessful = ProcessLightActorData(ActorData, LightActorElement);
		}
		else if (ActorElement->IsA(EDatasmithElementType::Camera))
		{
			IDatasmithCameraActorElement* CameraElement = static_cast<IDatasmithCameraActorElement*>(ActorElement.Get());

			bProcessSuccessful = ProcessCameraActorData(ActorData, CameraElement);
		}

		if (!bProcessSuccessful)
		{
			CreateActorComponent(ActorData, ActorElement);
			ActorData.AddState(EAssetState::Processed | EAssetState::Completed);
		}
	}

	bool FSceneImporter::DeleteComponent(FActorData& ActorData)
	{
		// Remove actor from the asset's list of referencers
		if (ActorData.AssetId != INDEX_NONE && AssetDataList.Contains(ActorData.AssetId))
		{
			RemoveFromReferencer(&AssetDataList[ActorData.AssetId].ElementId, ActorData.ElementId);
		}

		if (USceneComponent* SceneComponent = ActorData.GetObject<USceneComponent>())
		{
			if (ImportOptions.BuildHierarchy !=  EBuildHierarchyMethod::None)
			{
				USceneComponent* ParentComponent = SceneComponent->GetAttachmentRoot();

				AActor* Actor = SceneComponent->GetOwner();

				if (ParentComponent->GetOwner() == Actor)
				{
					TArray<USceneComponent*> Children = SceneComponent->GetAttachChildren();

					for (USceneComponent* ChildComponent : Children)
					{
						ChildComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
						ChildComponent->AttachToComponent(ParentComponent, FAttachmentTransformRules::KeepRelativeTransform);
					}

					DeleteSceneComponent(SceneComponent);
				}
				else
				{
					AActor* ParentActor = ParentComponent->GetOwner();

					TArray<AActor*> ChildActors;
					Actor->GetAttachedActors(ChildActors);

					for (AActor* ChildActor : ChildActors)
					{
						ChildActor->DetachFromActor(FDetachmentTransformRules::KeepRelativeTransform);
						ChildActor->AttachToActor(ParentActor, FAttachmentTransformRules::KeepRelativeTransform);
					}

					RootComponent->GetOwner()->GetWorld()->EditorDestroyActor(Actor, true);
				}
			}
			else if (SceneComponent->GetAttachmentRoot() == RootComponent.Get())
			{
				DeleteSceneComponent(SceneComponent);
			}

			ActorData.Object.Reset();

			TasksToComplete |= EWorkerTask::GarbageCollect;

			return true;
		}

		return false;
	}

	bool FSceneImporter::DeleteData()
	{
		// Terminate all rendering commands before deleting any component and asset
		FlushRenderingCommands();

		bool bGarbageCollect = false;

		TArray<AActor*> ChildActors;
		RootComponent->GetOwner()->GetAttachedActors(ChildActors);

		// Check to see if last import required to build the actor's hierarchy
		bool bHasHierarchy = Algo::AnyOf(ChildActors, [](const AActor* ChildActor){ return ChildActor->ActorHasTag(RuntimeTag); });

		// If last import created actors, delete them from bottom to top
		if (bHasHierarchy)
		{
			// Remove hold onto scene component as they will be deleted along with the owning actor.
			for (TPair< FSceneGraphId, FActorData >& Pair : ActorDataList)
			{
				Pair.Value.Object.Reset();
			}

			UWorld* GameWorld = RootComponent->GetOwner()->GetWorld();

			for (AActor* ChildActor : ChildActors)
			{
				// Only delete actors created by runtime import
				if (ChildActor->ActorHasTag(RuntimeTag))
				{
					ChildActor->DetachFromActor(FDetachmentTransformRules::KeepRelativeTransform);
					DeleteActorRecursive(GameWorld, ChildActor);
				}
			}

			TArray<USceneComponent*> SceneComponents = RootComponent->GetAttachChildren();
			for (USceneComponent* SceneComponent : SceneComponents)
			{
				// Only delete components created by runtime import
				if (SceneComponent->ComponentHasTag(RuntimeTag))
				{
					DeleteSceneComponent(SceneComponent);
				}
			}

			bGarbageCollect = true;
		}
		else
		{
			for (TPair< FSceneGraphId, FActorData >& Pair : ActorDataList)
			{
				bGarbageCollect |= DeleteComponent(Pair.Value);
			}
		}

		for (TPair< FSceneGraphId, FAssetData >& Entry : AssetDataList)
		{
			bGarbageCollect |= DeleteAsset(Entry.Value);
		}

		return bGarbageCollect;
	}

	bool FSceneImporter::ProcessCameraActorData(FActorData& ActorData, IDatasmithCameraActorElement* CameraElement)
	{
		if (ActorData.HasState(EAssetState::Processed) || ActorData.Object.IsValid())
		{
			ActorData.AddState(EAssetState::Processed);
			return true;
		}

		if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None)
		{
			UCineCameraComponent* CameraComponent = ActorData.GetObject<UCineCameraComponent>();

			if (CameraComponent == nullptr)
			{
				if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None)
				{
					UWorld* World = RootComponent->GetOwner()->GetWorld();

					ACineCameraActor* CameraActor = Cast< ACineCameraActor >( World->SpawnActor( ACineCameraActor::StaticClass(), nullptr, nullptr ) );
					RenameObject(CameraActor, CameraElement->GetName());
#if WITH_EDITOR
					CameraActor->SetActorLabel(CameraElement->GetLabel());
#endif
					CameraComponent = CameraActor->GetCineCameraComponent();
				}
				else
				{
					FName ComponentName = NAME_None;
					ComponentName = MakeUniqueObjectName(RootComponent->GetOwner(), UCineCameraComponent::StaticClass(), CameraElement->GetLabel());
					CameraComponent = NewObject< UCineCameraComponent >(RootComponent->GetOwner(), ComponentName);
				}

				ActorData.Object = CameraComponent;
			}
			//else
			//{
			//	CameraComponent->GetOwner()->UpdateComponentTransforms();
			//	CameraComponent->GetOwner()->MarkComponentsRenderStateDirty();
			//}

			CameraComponent->Filmback.SensorWidth = CameraElement->GetSensorWidth();
			CameraComponent->Filmback.SensorHeight = CameraElement->GetSensorWidth() / CameraElement->GetSensorAspectRatio();
			CameraComponent->LensSettings.MaxFStop = 32.0f;
			CameraComponent->CurrentFocalLength = CameraElement->GetFocalLength();
			CameraComponent->CurrentAperture = CameraElement->GetFStop();

			CameraComponent->FocusSettings.FocusMethod = CameraElement->GetEnableDepthOfField() ? ECameraFocusMethod::Manual : ECameraFocusMethod::DoNotOverride;
			CameraComponent->FocusSettings.ManualFocusDistance = CameraElement->GetFocusDistance();

			if (const IDatasmithPostProcessElement* PostProcess = CameraElement->GetPostProcess().Get())
			{
				FPostProcessSettings& PostProcessSettings = CameraComponent->PostProcessSettings;

				if ( !FMath::IsNearlyEqual( PostProcess->GetTemperature(), 6500.f ) )
				{
					PostProcessSettings.bOverride_WhiteTemp = true;
					PostProcessSettings.WhiteTemp = PostProcess->GetTemperature();
				}

				if ( PostProcess->GetVignette() > 0.f )
				{
					PostProcessSettings.bOverride_VignetteIntensity = true;
					PostProcessSettings.VignetteIntensity = PostProcess->GetVignette();
				}

				if ( !FMath::IsNearlyEqual( PostProcess->GetSaturation(), 1.f ) )
				{
					PostProcessSettings.bOverride_ColorSaturation = true;
					PostProcessSettings.ColorSaturation.W = PostProcess->GetSaturation();
				}

				if ( PostProcess->GetCameraISO() > 0.f || PostProcess->GetCameraShutterSpeed() > 0.f || PostProcess->GetDepthOfFieldFstop() > 0.f )
				{
					PostProcessSettings.bOverride_AutoExposureMethod = true;
					PostProcessSettings.AutoExposureMethod = EAutoExposureMethod::AEM_Manual;

					if ( PostProcess->GetCameraISO() > 0.f )
					{
						PostProcessSettings.bOverride_CameraISO = true;
						PostProcessSettings.CameraISO = PostProcess->GetCameraISO();
					}

					if ( PostProcess->GetCameraShutterSpeed() > 0.f )
					{
						PostProcessSettings.bOverride_CameraShutterSpeed = true;
						PostProcessSettings.CameraShutterSpeed = PostProcess->GetCameraShutterSpeed();
					}

					if ( PostProcess->GetDepthOfFieldFstop() > 0.f )
					{
						PostProcessSettings.bOverride_DepthOfFieldFstop = true;
						PostProcessSettings.DepthOfFieldFstop = PostProcess->GetDepthOfFieldFstop();
					}
				}
			}
		}

		FinalizeComponent(ActorData);

		ActorData.SetState(EAssetState::Processed | EAssetState::Completed);

		return true;
	}

	FActorData& FSceneImporter::FindOrAddActorData(const TSharedPtr<IDatasmithActorElement>& ActorElement)
	{
		FSceneGraphId ElementId = ActorElement->GetNodeId();

		if (!Elements.Contains(ElementId))
		{
			Elements.Add(ElementId, ActorElement);

			FActorData ActorData(ElementId);
			return ActorDataList.Emplace(ElementId, MoveTemp(ActorData));
		}

		ensure(ActorDataList.Contains(ElementId));
		return ActorDataList[ElementId];
	}

	void FSceneImporter::CreateActorComponent(FActorData& ActorData, const TSharedPtr< IDatasmithActorElement >& ActorElement)
	{
		const bool bBuildHierarchy = ImportOptions.BuildHierarchy == EBuildHierarchyMethod::Unfiltered || (ImportOptions.BuildHierarchy == EBuildHierarchyMethod::Simplified && ActorElement->GetChildrenCount() > 0);
		if (!bBuildHierarchy)
		{
			ActorData.SetState(EAssetState::Skipped);
			return;
		}

		bool bMustProcess = true;

		if (ImportOptions.BuildHierarchy == EBuildHierarchyMethod::Simplified)
		{
			TFunction<bool(const TSharedPtr< IDatasmithActorElement >&)> MustKeep;
			MustKeep = [this, &MustKeep](const TSharedPtr< IDatasmithActorElement >& ActorElementToCheck) -> bool
			{
				for (int32 Index = 0; Index < ActorElementToCheck->GetChildrenCount(); ++Index)
				{
					if (MustKeep(ActorElementToCheck->GetChild(Index)))
					{
						return true;
					}
				}

				const bool bMustKeep = ActorElementToCheck->IsA(EDatasmithElementType::Camera)
					|| ActorElementToCheck->IsA(EDatasmithElementType::Light)
					|| ActorElementToCheck->IsA(EDatasmithElementType::StaticMeshActor);

				if (!bMustKeep)
				{
					FActorData& LocalActorData = this->FindOrAddActorData(ActorElementToCheck);
					LocalActorData.SetState(EAssetState::Processed | EAssetState::Completed | EAssetState::Skipped);
				}

				return bMustKeep;
			};

			bMustProcess = MustKeep(ActorElement);

			if (bMustProcess && ActorElement->GetChildrenCount() == 1 && ActorElement->GetChild(0)->GetChildrenCount() == 0)
			{
				ActorElement->GetChild(0)->SetIsAComponent(false);
				ActorData.SetState(EAssetState::Skipped);
				bMustProcess = false;
			}
		}

		if(bMustProcess)
		{
			USceneComponent* SceneComponent = ActorData.GetObject<USceneComponent>();

			if (SceneComponent == nullptr)
			{
				AActor* Actor = RootComponent->GetOwner()->GetWorld()->SpawnActor(AActor::StaticClass(), nullptr, nullptr);
				RenameObject(Actor, ActorElement->GetName());
#if WITH_EDITOR
				Actor->SetActorLabel( ActorElement->GetLabel() );
#endif

				Actor->Tags.Empty(ActorElement->GetTagsCount());
				for (int32 Index = 0; Index < ActorElement->GetTagsCount(); ++Index)
				{
					Actor->Tags.Add(ActorElement->GetTag(Index));
				}

				// #ue_datasmithruntime: Add Layers and metadata

				SceneComponent = Actor->GetRootComponent();

				if ( !SceneComponent )
				{
					SceneComponent = NewObject< USceneComponent >( Actor, USceneComponent::StaticClass(), ActorElement->GetLabel(), RF_Transactional );

					Actor->AddInstanceComponent( SceneComponent );
					Actor->SetRootComponent( SceneComponent );
				}

				ActorData.Object = TWeakObjectPtr<UObject>(SceneComponent);
			}
			else if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None && SceneComponent->GetOwner()->GetRootComponent() == SceneComponent)
			{
				RenameObject(SceneComponent->GetOwner(), ActorElement->GetName());
			}

			FinalizeComponent(ActorData);
		}
	}

	void FSceneImporter::FinalizeComponent(FActorData& ActorData)
	{
		USceneComponent* SceneComponent = ActorData.GetObject<USceneComponent>();
		if (!SceneComponent)
		{
			return;
		}

		if (SceneComponent->IsRegistered())
		{
			SceneComponent->UnregisterComponent();
		}

		// Find the right parent
		FSceneGraphId ParentId = ActorData.ParentId;
		while (ParentId != DirectLink::InvalidId && ActorDataList[ParentId].HasState(EAssetState::Skipped))
		{
			ParentId = ActorDataList[ParentId].ParentId;
		}

		if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None && ParentId != DirectLink::InvalidId)
		{
			// If hierarchy building is required, wait for parent component to be created if applicable
			if (ActorDataList[ParentId].GetObject<USceneComponent>() == nullptr)
			{
				FActionTaskFunction FinalizeComponentFunc = [this,ParentId](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
				{
					FActorData& ActorData = ActorDataList[Referencer.GetId()];

					if (ActorDataList[ParentId].GetObject<USceneComponent>() == nullptr)
					{
						return EActionResult::Retry;
					}

					this->FinalizeComponent(ActorData);

					return EActionResult::Succeeded;
				};

				AddToQueue(EQueueTask::NonAsyncQueue, { FinalizeComponentFunc, { EDataType::Actor, ActorData.ElementId, 0 } });
				TasksToComplete |= EWorkerTask::ComponentFinalize;

				return;
			}
		}

		const bool bUseParent = ParentId != DirectLink::InvalidId && ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None;
		USceneComponent* ParentComponent = bUseParent ? ActorDataList[ParentId].GetObject<USceneComponent>() : RootComponent.Get();

		if (ParentComponent)
		{
			const IDatasmithActorElement* ActorElement = static_cast<IDatasmithActorElement*>(Elements[ActorData.ElementId].Get());

			const FTransform ParentToWorld = ParentComponent->GetComponentToWorld();
			const FTransform RelativeTransform = ActorData.WorldTransform.GetRelativeTransform(ParentToWorld);

			SceneComponent->SetRelativeTransform( RelativeTransform );
			SceneComponent->SetVisibility(ActorElement->GetVisibility());
			SceneComponent->SetMobility( EComponentMobility::Movable );

			if (SceneComponent->GetAttachParent() != ParentComponent)
			{
				SceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
				SceneComponent->AttachToComponent( ParentComponent, FAttachmentTransformRules::KeepRelativeTransform );
			}

			SceneComponent->ComponentTags.Empty(ActorElement->GetTagsCount());
			for (int32 Index = 0; Index < ActorElement->GetTagsCount(); ++Index)
			{
				SceneComponent->ComponentTags.Add(ActorElement->GetTag(Index));
			}

			if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None)
			{
				if (ActorElement->IsAComponent())
				{
					RenameObject(SceneComponent, ActorElement->GetName(), ParentComponent->GetOwner());
				}
				else if (SceneComponent->GetOwner() == RootComponent->GetOwner() && SceneComponent->GetOwner() != ParentComponent->GetOwner())
				{
					RenameObject(SceneComponent, ActorElement->GetName(), ParentComponent->GetOwner());
				}

				SceneComponent->GetOwner()->AddInstanceComponent(SceneComponent);

				// Add runtime tag on actor created on import.
				if (SceneComponent->GetOwner() != RootComponent->GetOwner() && !SceneComponent->GetOwner()->ActorHasTag(RuntimeTag))
				{
					SceneComponent->GetOwner()->Tags.Add(RuntimeTag);
				}
			}

			SceneComponent->RegisterComponentWithWorld(RootComponent->GetOwner()->GetWorld());
		}
		else
		{
			ensure(false);
		}

		ApplyMetadata(ActorData.MetadataId, SceneComponent);
	}

	void DeleteActorRecursive(UWorld* GameWorld, AActor* Actor)
	{
		TArray<AActor*> ChildActors;
		Actor->GetAttachedActors(ChildActors);

		for (AActor* ChildActor : ChildActors)
		{
			if (ChildActor->ActorHasTag(RuntimeTag))
			{
				ChildActor->DetachFromActor(FDetachmentTransformRules::KeepRelativeTransform);
				DeleteActorRecursive(GameWorld, ChildActor);
			}
		}

		if (USceneComponent* RootComponent = Actor->GetRootComponent())
		{
			TArray<USceneComponent*> SceneComponents = RootComponent->GetAttachChildren();
			for (USceneComponent* SceneComponent : SceneComponents)
			{
				// Only delete components created by runtime import
				if (SceneComponent->ComponentHasTag(RuntimeTag))
				{
					DeleteSceneComponent(SceneComponent);
				}
			}
		}

		ensure(GameWorld->DestroyActor(Actor));

		// Since deletion can be delayed, rename to avoid future name collision
		// Call UObject::Rename directly on actor to avoid AActor::Rename which unnecessarily sunregister and re-register components
		Actor->UObject::Rename( nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
	}

	void HideSceneComponent(USceneComponent* SceneComponent)
	{
		if (SceneComponent)
		{
			SceneComponent->SetVisibility(false);

			if (UStaticMeshComponent* MeshComponent = Cast< UStaticMeshComponent >(SceneComponent))
			{
				MeshComponent->OverrideMaterials.Reset();
				MeshComponent->SetStaticMesh(nullptr);
				MeshComponent->ReleaseResources();
			}

			SceneComponent->MarkRenderStateDirty();
			SceneComponent->MarkRenderDynamicDataDirty();
		}
	}

	void DeleteSceneComponent(USceneComponent* SceneComponent)
	{
		SceneComponent->UnregisterComponent();

		SceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);

		SceneComponent->ClearFlags(RF_AllFlags);
		SceneComponent->SetFlags(RF_Transient);
		//SceneComponent->Rename(nullptr, nullptr, REN_NonTransactional | REN_DontCreateRedirectors);
		SceneComponent->MarkAsGarbage();
	}

	void RenameObject(UObject* Object, const TCHAR* DesiredName, UObject* NewOwner)
	{
		FString ObjectName(DesiredName);

		// Validate desired name does not collide
		if (!ObjectName.IsEmpty())
		{
			if (!Object->Rename(DesiredName, NewOwner, REN_Test))
			{
				ObjectName = MakeUniqueObjectName(Object->GetOuter(), Object->GetClass(), DesiredName).ToString();
			}
		}

		Object->Rename(ObjectName.IsEmpty() ? nullptr : *ObjectName, NewOwner, REN_NonTransactional | REN_DontCreateRedirectors);
	}

	USceneComponent* CreateComponent(FActorData& ActorData, UClass* Class, AActor* Owner)
	{
		USceneComponent* SceneComponent = ActorData.GetObject<USceneComponent>();

		if (SceneComponent == nullptr)
		{
			SceneComponent = NewObject< USceneComponent >(Owner, Class, NAME_None);
			if (SceneComponent == nullptr)
			{
				return nullptr;
			}

			SceneComponent->SetMobility(EComponentMobility::Movable);

			// Add runtime tag to scene component
			SceneComponent->ComponentTags.Add(RuntimeTag);

			ActorData.Object = TWeakObjectPtr<UObject>(SceneComponent);
		}

		return SceneComponent;
	}

} // End of namespace DatasmithRuntime
