// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithRuntime.h"

#include "DatasmithRuntimeBlueprintLibrary.h"
#include "DirectLinkUtils.h"
#include "DatasmithTranslatableSource.h"
#include "LogCategory.h"
#include "SceneImporter.h"

#include "DirectLinkSceneSnapshot.h"

#include "DatasmithSceneFactory.h"
#include "DatasmithTranslator.h"

#include "Async/Async.h"
#include "Misc/Paths.h"

#include "Misc/Paths.h"

#ifdef ASSET_DEBUG
#include "DatasmithSceneXmlWriter.h"
#include "HAL/FileManager.h"
#endif

const FBoxSphereBounds DefaultBounds(FVector::ZeroVector, FVector(2000), 1000);
const TCHAR* EmptyScene = TEXT("Nothing Loaded");

// Use to force sequential update of game content
std::atomic_bool ADatasmithRuntimeActor::bImportingScene(false);

TUniquePtr<DatasmithRuntime::FTranslationThread> ADatasmithRuntimeActor::TranslationThread;

void ADatasmithRuntimeActor::OnStartupModule()
{
	TranslationThread = MakeUnique<DatasmithRuntime::FTranslationThread>();
}

void ADatasmithRuntimeActor::OnShutdownModule()
{
	TranslationThread.Reset();
}

ADatasmithRuntimeActor::ADatasmithRuntimeActor()
	: LoadedScene(EmptyScene)
	, bNewScene(false)
	, bReceivingStarted(false)
	, bReceivingEnded(false)
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("DatasmithRuntimeComponent"));
	AddInstanceComponent( RootComponent );
	RootComponent->SetMobility(EComponentMobility::Movable);
	RootComponent->Bounds = DefaultBounds;

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
	PrimaryActorTick.TickInterval = 0.1f;
}

void ADatasmithRuntimeActor::Tick(float DeltaTime)
{
	if (SceneElement.IsValid() && bReceivingStarted && bReceivingEnded)
	{
		UE_LOGF(LogDatasmithRuntime, Log, "ADatasmithRuntimeActor::Tick - Process scene's changes");

		// Prevent any other DatasmithRuntime actors to import concurrently
		if (!bImportingScene.exchange(true))
		{
			if (Translator.IsValid())
			{
				SceneImporter->SetTranslator(Translator);
				ApplyNewScene();
			}
			else if (bNewScene == true)
			{
				ApplyNewScene();
			}
			else
			{
				bBuilding = true;

				SceneImporter->IncrementalUpdate(SceneElement.ToSharedRef(), UpdateContext);
				UpdateContext.Additions.Empty();
				UpdateContext.Deletions.Empty();
				UpdateContext.Updates.Empty();

				SceneElement.Reset();
			}

			bReceivingStarted = false;
			bReceivingEnded = false;
		}
	}

	Super::Tick(DeltaTime);
}

void ADatasmithRuntimeActor::BeginPlay()
{
	Super::BeginPlay();

	// Create scene importer
	SceneImporter = MakeShared< DatasmithRuntime::FSceneImporter >( this );

	// Register to DirectLink
	DirectLinkHelper = MakeShared< DatasmithRuntime::FDestinationProxy >( this );
	DirectLinkHelper->RegisterDestination(*GetName());
}

void ADatasmithRuntimeActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Reset();

	// Unregister to DirectLink
	DirectLinkHelper->UnregisterDestination();
	DirectLinkHelper.Reset();

	// Delete scene importer
	SceneImporter.Reset();
	SceneElement.Reset();
	Translator.Reset();

	Super::EndPlay(EndPlayReason);
}

void ADatasmithRuntimeActor::OnOpenDelta(/*int32 ElementsCount*/)
{
	if (!IsInGameThread())
	{
		// Block the DirectLink thread, if we are still processing the previous delta
		while (bReceivingStarted)
		{
			FPlatformProcess::SleepNoStats(0.1f);
		}
	}

	UE_LOGF(LogDatasmithRuntime, Log, "ADatasmithRuntimeActor::OnOpenDelta");
	bNewScene = false;
	bReceivingStarted = Translator.IsValid() || DirectLinkHelper.IsValid();
	if (DirectLinkHelper.IsValid())
	{
		SceneElement = DirectLinkHelper->GetScene();
	}
	bReceivingEnded = false;
	ElementDeltaStep = /*ElementsCount > 0 ? 1.f / (float)ElementsCount : 0.f*/0.f;
}

void ADatasmithRuntimeActor::OnNewScene(const DirectLink::FSceneIdentifier& SceneId)
{
	UE_LOGF(LogDatasmithRuntime, Log, "ADatasmithRuntimeActor::OnNewScene");
	bNewScene = true;
}

void ADatasmithRuntimeActor::OnAddElement(DirectLink::FSceneGraphId ElementId, TSharedPtr<IDatasmithElement> Element)
{
	Progress += ElementDeltaStep;
	if (bNewScene == false)
	{
		UpdateContext.Additions.Add(Element);
	}
}

void ADatasmithRuntimeActor::OnRemovedElement(DirectLink::FSceneGraphId ElementId)
{
	Progress += ElementDeltaStep;
	UpdateContext.Deletions.Add(ElementId);
}

void ADatasmithRuntimeActor::OnChangedElement(DirectLink::FSceneGraphId ElementId, TSharedPtr<IDatasmithElement> Element)
{
	Progress += ElementDeltaStep;
	UpdateContext.Updates.Add(Element);
}

bool ADatasmithRuntimeActor::IsConnected()
{
	return DirectLinkHelper.IsValid() ? DirectLinkHelper->IsConnected() : false;
}

FString ADatasmithRuntimeActor::GetSourceName()
{
	return DirectLinkHelper.IsValid() ? DirectLinkHelper->GetSourceName() : FString();
}

bool ADatasmithRuntimeActor::OpenConnectionWithIndex(int32 SourceIndex)
{
	using namespace DatasmithRuntime;

	if (DirectLinkHelper.IsValid() && DirectLinkHelper->CanConnect())
	{
		const TArray<FDatasmithRuntimeSourceInfo>& SourcesList = FDestinationProxy::GetListOfSources();

		if (SourcesList.IsValidIndex(SourceIndex))
		{
			return DirectLinkHelper->OpenConnection(SourcesList[SourceIndex].SourceHandle);
		}
		else if (SourceIndex == INDEX_NONE)
		{
			CloseConnection();
			Reset();
			return true;
		}
	}

	return false;
}

int32 ADatasmithRuntimeActor::GetSourceIndex()
{
	using namespace DatasmithRuntime;

	if (DirectLinkHelper.IsValid() && DirectLinkHelper->IsConnected())
	{
		const TArray<FDatasmithRuntimeSourceInfo>& SourcesList = FDestinationProxy::GetListOfSources();

		const int32 SourceIndex = SourcesList.IndexOfByPredicate(
			[SourceHandle = DirectLinkHelper->GetConnectedSourceHandle()](const FDatasmithRuntimeSourceInfo& SourceInfo) -> bool
			{
				return SourceInfo.SourceHandle == SourceHandle;
			});

		return SourceIndex;
	}

	return INDEX_NONE;
}

void ADatasmithRuntimeActor::CloseConnection()
{
	if (DirectLinkHelper.IsValid() && DirectLinkHelper->IsConnected())
	{
		DirectLinkHelper->CloseConnection();
		Reset();
	}
}

void ADatasmithRuntimeActor::OnCloseDelta()
{
	// Something is wrong
	if (!bReceivingStarted)
	{
		ensure(false);
		return;
	}

	bReceivingEnded = DirectLinkHelper.IsValid() || Translator.IsValid();
}

void ADatasmithRuntimeActor::ApplyNewScene()
{
	TRACE_BOOKMARK(TEXT("Load started - %s"), SceneElement->GetName());

	UE_LOGF(LogDatasmithRuntime, Log, "ADatasmithRuntimeActor::ApplyNewScene");

	SceneImporter->Reset(true);

	RootComponent->Bounds = DefaultBounds;

	bBuilding = true;
	Progress = 0.f;
	LoadedScene = SceneElement->GetName();
	SceneImporter->StartImport( SceneElement.ToSharedRef(), ImportOptions );

	SceneElement.Reset();
}

void ADatasmithRuntimeActor::Reset()
{
	SceneImporter->Reset(true);

	// Reset called while importing a scene, update flag accordingly
	if (bBuilding || bReceivingStarted)
	{
		bImportingScene = false;
	}

	bReceivingStarted = false;
	bReceivingEnded = false;

	bBuilding = false;
	Progress = 0.f;
	LoadedScene = EmptyScene;

	RootComponent->Bounds = DefaultBounds;
}

void ADatasmithRuntimeActor::OnImportEnd()
{
	Translator.Reset();

	bBuilding = false;

	// Allow any other DatasmithRuntime actors to import concurrently
	bImportingScene = false;

	bReceivingStarted = false;
	bReceivingEnded = false;
}

bool ADatasmithRuntimeActor::LoadFile(const FString& FilePath)
{
	if( !FPaths::FileExists( FilePath ) )
	{
		return false;
	}

	// Wait for any ongoing import to complete
	// #ue_datasmithruntime: To do add code to interrupt 
	while (bReceivingStarted)
	{
		FPlatformProcess::SleepNoStats(0.1f);
	}

	// Temporarily manually allow only udatasmith and GLTF file formats
	FString Extension = FPaths::GetExtension(FilePath);
	if (!(Extension.Equals(TEXT("udatasmith"), ESearchCase::IgnoreCase) ||
		Extension.Equals(TEXT("gltf"), ESearchCase::IgnoreCase) ||
		Extension.Equals(TEXT("glb"), ESearchCase::IgnoreCase)
		))
	{
		UE_LOGF(LogDatasmithRuntime, Log, "Extension %ls is not supported.", *Extension);
		return false;
	}

	CloseConnection();

	Progress = 0.f;

	if (!TranslationThread->ThreadResult.IsValid())
	{
		TranslationThread->ThreadResult = Async(EAsyncExecution::Thread,
			[&]() -> void
			{
				FPlatformProcess::SetThreadName(TEXT("RuntimeTranslation"));
				TranslationThread->bKeepRunning = true;
				TranslationThread->ThreadEvent = FPlatformProcess::GetSynchEventFromPool();
				TranslationThread->Run();
			}
		);
	}

	TranslationThread->AddJob({ this, FilePath });

	// Set all import options to defaults for DatasmithRuntime
	return true;
}

namespace DatasmithRuntime
{
	bool FTranslationJob::Execute()
	{
		if (!RuntimeActor.IsValid())
		{
			return false;
		}

		FDatasmithSceneSource Source;
		Source.SetSourceFile(FilePath);

		RuntimeActor->ExternalFile.Empty();

		FDatasmithTranslatableSceneSource TranslatableSceneSource(Source);
		if (!TranslatableSceneSource.IsTranslatable())
		{
			RuntimeActor->LoadedScene = Source.GetSourceFileExtension() + TEXT(" file format is not supported");
			return false;
		}

		TSharedPtr<IDatasmithTranslator> Translator = TranslatableSceneSource.GetTranslator();
		if (!Translator.IsValid())
		{
			RuntimeActor->LoadedScene = Source.GetSourceFileExtension() + TEXT(" file format is not supported");
			return false;
		}

		while(RuntimeActor->IsReceiving() || ADatasmithRuntimeActor::bImportingScene)
		{
			FPlatformProcess::Sleep(0.05f);
		}

		RuntimeActor->OnOpenDelta();

		RuntimeActor->LoadedScene = Source.GetSceneName();

		TSharedRef<IDatasmithScene> SceneElement = FDatasmithSceneFactory::CreateScene(*RuntimeActor->LoadedScene);

		if (!Translator->LoadScene( SceneElement ))
		{

			RuntimeActor->OnImportEnd();

			RuntimeActor->SceneElement = nullptr;
			RuntimeActor->LoadedScene = TEXT("Loading failed");

			return false;
		}

		DirectLink::BuildIndexForScene(&SceneElement.Get());

		RuntimeActor->SceneElement = SceneElement;
		RuntimeActor->Translator = Translator;
		RuntimeActor->ExternalFile = FilePath;

		RuntimeActor->OnCloseDelta();

		return true;
	}

	void FTranslationThread::Run()
	{
		while (bKeepRunning)
		{
			FTranslationJob TranslationJob;
			if (JobQueue.Dequeue(TranslationJob))
			{
				TranslationJob.Execute();
				continue;
			}

			FPlatformProcess::Sleep(0.1f);
		}

		// The FTranslationThread is being deleted, trigger the event before exiting
		ThreadEvent->Trigger();
	}

	FTranslationThread::~FTranslationThread()
	{
		if (bKeepRunning)
		{
			bKeepRunning = false;
			ThreadResult.Reset();
			// Wait for FTranslationThread::Run() to be completed
			ThreadEvent->Wait();
			FPlatformProcess::ReturnSynchEventToPool(ThreadEvent);
		}
	}
}
