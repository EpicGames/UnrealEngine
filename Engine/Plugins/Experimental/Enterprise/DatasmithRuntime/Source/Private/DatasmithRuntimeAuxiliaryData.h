// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Templates/SubclassOf.h"


#include "DatasmithRuntimeAuxiliaryData.generated.h"

namespace DatasmithRuntime
{
	struct FAssetData;
}

/**
 * Utility class to hold on FAssetData entities while their associated texture is being built
 */
UCLASS(MinimalAPI)
class UDatasmithRuntimeTHelper : public UObject, public IInterface_AssetUserData
{
	GENERATED_BODY()

public:
	//~ Begin IInterface_AssetUserData Interface
	virtual void AddAssetUserData( UAssetUserData* InUserData ) override
	{
		if ( InUserData != NULL )
		{
			UAssetUserData* ExistingData = GetAssetUserDataOfClass( InUserData->GetClass() );
			if ( ExistingData != NULL )
			{
				AssetUserData.Remove( ExistingData );
			}

			AssetUserData.Add( InUserData );
		}
	}

	virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override
	{
		for ( int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++ )
		{
			UAssetUserData* Datum = AssetUserData[DataIdx];
			if ( Datum != NULL && Datum->IsA(InUserDataClass ) )
			{
				AssetUserData.RemoveAt( DataIdx );
				return;
			}
		}
	}

	virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override
	{
		for ( int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++ )
		{
			UAssetUserData* Datum = AssetUserData[DataIdx];
			if ( Datum != NULL && Datum->IsA( InUserDataClass ) )
			{
				return Datum;
			}
		}

		return nullptr;
	}

	virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override
	{
		return &ToRawPtrTArrayUnsafe(AssetUserData);
	}
	//~ End IInterface_AssetUserData Interface

private:
	/** Array of user data stored with the asset */
	UPROPERTY()
	TArray< TObjectPtr<UAssetUserData> > AssetUserData;
};

/** Asset user data that can be used with DatasmithRuntime on Actors and other objects  */
UCLASS(MinimalAPI)
class UDatasmithRuntimeAuxiliaryData : public UAssetUserData
{
	GENERATED_BODY()

public:
	UDatasmithRuntimeAuxiliaryData()
		: Auxiliary(nullptr)
		, bIsCompleted(false)
	{
	}

	UPROPERTY(VisibleAnywhere, Category = "DatasmithRuntime Internal")
	TObjectPtr<UObject> Auxiliary;

	TSet<uint64> Referencers;

	std::atomic_bool bIsCompleted;
};
