// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithRuntimeBlueprintLibrary.h"

#include "DatasmithRuntime.h"
#include "DirectLinkUtils.h"

#if WITH_EDITOR
#include "DesktopPlatformModule.h"
#include "Engine/GameViewportClient.h"
#include "IDesktopPlatform.h"
#include "Engine/Engine.h"
#include "Widgets/SWindow.h"
#elif PLATFORM_WINDOWS
#include "HAL/FileManager.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Microsoft/COMPointer.h"
#include "Misc/Paths.h"
#include <commdlg.h>
#include <shlobj.h>
#include <Winver.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif


class UDirectLinkProxy;

bool UDatasmithRuntimeLibrary::LoadFile(ADatasmithRuntimeActor* DatasmithRuntimeActor, const FString& FilePath)
{
	return DatasmithRuntimeActor ? DatasmithRuntimeActor->LoadFile(FilePath) : false;
}

void UDatasmithRuntimeLibrary::ResetActor(ADatasmithRuntimeActor* DatasmithRuntimeActor)
{
	if (DatasmithRuntimeActor)
	{
		DatasmithRuntimeActor->Reset();
	}
}

UDirectLinkProxy* UDatasmithRuntimeLibrary::GetDirectLinkProxy()
{
	return DatasmithRuntime::GetDirectLinkProxy();
}

bool UDatasmithRuntimeLibrary::LoadFileFromExplorer(ADatasmithRuntimeActor* DatasmithRuntimeActor, const FString& DefaultPath)
{
	if (DatasmithRuntimeActor == nullptr)
	{
		return false;
	}

	TArray<FString> OutFilenames;

	FString	FileTypes = TEXT("All Files (*.udatasmith;*.gltf;*.glb)|*.udatasmith;*.gltf;*.glb|Datasmith files (*.udatasmith)|*.udatasmith|GL Transmission Format (*.gltf;*.glb)|*.gltf;*.glb");

#if WITH_EDITOR
	if (GEngine && GEngine->GameViewport)
	{
		void* ParentWindowHandle = GEngine->GameViewport->GetWindow()->GetNativeWindow()->GetOSWindowHandle();
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform)
		{
			//Opening the file picker!
			uint32 SelectionFlag = 0; //A value of 0 represents single file selection while a value of 1 represents multiple file selection
			DesktopPlatform->OpenFileDialog(ParentWindowHandle, TEXT("Choose A File"), DefaultPath, FString(""), FileTypes, SelectionFlag, OutFilenames);
		}
	}
#elif PLATFORM_WINDOWS
	TComPtr<IFileDialog> FileDialog;
	if (SUCCEEDED(::CoCreateInstance(CLSID_FileOpenDialog, nullptr, CLSCTX_INPROC_SERVER, IID_IFileOpenDialog, IID_PPV_ARGS_Helper(&FileDialog))))
	{
		// Set up common settings
		FileDialog->SetTitle(TEXT("Choose A File"));
		if (!DefaultPath.IsEmpty())
		{
			// SHCreateItemFromParsingName requires the given path be absolute and use \ rather than / as our normalized paths do
			FString DefaultWindowsPath = FPaths::ConvertRelativePathToFull(DefaultPath);
			DefaultWindowsPath.ReplaceInline(TEXT("/"), TEXT("\\"), ESearchCase::CaseSensitive);

			TComPtr<IShellItem> DefaultPathItem;
			if (SUCCEEDED(::SHCreateItemFromParsingName(*DefaultWindowsPath, nullptr, IID_PPV_ARGS(&DefaultPathItem))))
			{
				FileDialog->SetFolder(DefaultPathItem);
			}
		}

		// Set-up the file type filters
		TArray<FString> UnformattedExtensions;
		TArray<COMDLG_FILTERSPEC> FileDialogFilters;
		{
			const FString DefaultFileTypes = TEXT("All Files (*.*)|*.*");
			DefaultFileTypes.ParseIntoArray(UnformattedExtensions, TEXT("|"), true);

			if (UnformattedExtensions.Num() % 2 == 0)
			{
				FileDialogFilters.Reserve(UnformattedExtensions.Num() / 2);
				for (int32 ExtensionIndex = 0; ExtensionIndex < UnformattedExtensions.Num();)
				{
					COMDLG_FILTERSPEC& NewFilterSpec = FileDialogFilters[FileDialogFilters.AddDefaulted()];
					NewFilterSpec.pszName = *UnformattedExtensions[ExtensionIndex++];
					NewFilterSpec.pszSpec = *UnformattedExtensions[ExtensionIndex++];
				}
			}
		}
		FileDialog->SetFileTypes(FileDialogFilters.Num(), FileDialogFilters.GetData());

		// Show the picker
		if (SUCCEEDED(FileDialog->Show(NULL)))
		{
			int32 OutFilterIndex = 0;
			if (SUCCEEDED(FileDialog->GetFileTypeIndex((UINT*)&OutFilterIndex)))
			{
				OutFilterIndex -= 1; // GetFileTypeIndex returns a 1-based index
			}

			TFunction<void(const FString&)> AddOutFilename = [&OutFilenames](const FString& InFilename)
			{
				FString& OutFilename = OutFilenames.Add_GetRef(InFilename);
				OutFilename = IFileManager::Get().ConvertToRelativePath(*OutFilename);
				FPaths::NormalizeFilename(OutFilename);
			};

			{
				IFileOpenDialog* FileOpenDialog = static_cast<IFileOpenDialog*>(FileDialog.Get());

				TComPtr<IShellItemArray> Results;
				if (SUCCEEDED(FileOpenDialog->GetResults(&Results)))
				{
					DWORD NumResults = 0;
					Results->GetCount(&NumResults);
					for (DWORD ResultIndex = 0; ResultIndex < NumResults; ++ResultIndex)
					{
						TComPtr<IShellItem> Result;
						if (SUCCEEDED(Results->GetItemAt(ResultIndex, &Result)))
						{
							PWSTR pFilePath = nullptr;
							if (SUCCEEDED(Result->GetDisplayName(SIGDN_FILESYSPATH, &pFilePath)))
							{
								AddOutFilename(pFilePath);
								::CoTaskMemFree(pFilePath);
							}
						}
					}
				}
			}
		}
	}
#endif

	if (OutFilenames.Num() > 0)
	{
		return LoadFile( DatasmithRuntimeActor, OutFilenames[0]);
	}

	return false;
}
