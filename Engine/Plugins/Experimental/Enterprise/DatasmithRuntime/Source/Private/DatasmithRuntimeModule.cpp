// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithRuntimeModule.h"

#include "DatasmithRuntime.h"
#include "DirectLinkUtils.h"
#include "LogCategory.h"
#include "MaterialImportUtils.h"
#include "MaterialSelectors/DatasmithRuntimeMaterialSelector.h"

#include "DatasmithTranslatorModule.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"

#if WITH_EDITOR
#include "Settings/ProjectPackagingSettings.h"
#include "Misc/Paths.h"
#endif // WITH_EDITOR

const TCHAR* MaterialsPath = TEXT("/DatasmithRuntime/Materials");

class FDatasmithRuntimeModule : public IDatasmithRuntimeModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// Verify DatasmithTranslatorModule has been loaded
		check(IDatasmithTranslatorModule::IsAvailable());

#if WITH_EDITOR
		// If don't have any active references to our materials they won't be packaged into monolithic builds, and we wouldn't
		// be able to create dynamic material instances at runtime.
		UProjectPackagingSettings* PackagingSettings = Cast<UProjectPackagingSettings>( UProjectPackagingSettings::StaticClass()->GetDefaultObject() );
		if ( PackagingSettings )
		{
			bool bAlreadyInPath = false;

			TArray<FDirectoryPath>& DirectoriesToCook = PackagingSettings->DirectoriesToAlwaysCook;
			for ( int32 Index = DirectoriesToCook.Num() - 1; Index >= 0; --Index )
			{
				if ( FPaths::IsSamePath( DirectoriesToCook[ Index ].Path, MaterialsPath ) )
				{
					bAlreadyInPath = true;
					break;
				}
			}

			if ( !bAlreadyInPath )
			{
				FDirectoryPath MaterialsDirectory;
				MaterialsDirectory.Path = MaterialsPath;

				PackagingSettings->DirectoriesToAlwaysCook.Add( MaterialsDirectory );

				UE_LOGF(LogDatasmithRuntime, Log, "Adding %ls to the list of directories to always package otherwise we cannot create dynamic material instances at runtime", MaterialsPath);
			}
		}
#endif // WITH_EDITOR

		FModuleManager::Get().LoadModuleChecked(TEXT("UdpMessaging"));

		DatasmithRuntime::FDestinationProxy::InitializeEndpointProxy();

		FDatasmithReferenceMaterialManager::Get().RegisterSelector(DatasmithRuntime::MATERIAL_HOST, MakeShared< FDatasmithRuntimeMaterialSelector >());

		ADatasmithRuntimeActor::OnStartupModule();
	}

	virtual void ShutdownModule() override
	{
		ADatasmithRuntimeActor::OnShutdownModule();
		
		FDatasmithReferenceMaterialManager::Get().UnregisterSelector(DatasmithRuntime::MATERIAL_HOST);

		DatasmithRuntime::FDestinationProxy::ShutdownEndpointProxy();
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FDatasmithRuntimeModule, DatasmithRuntime);

