// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithRuntimeUtils.h"

#include "DatasmithRuntimeAuxiliaryData.h"

#include "DatasmithPayload.h"
#include "IDatasmithSceneElements.h"
#include "StaticMeshResources.h"
#include "Utility/DatasmithMeshHelper.h"

#include "Algo/AnyOf.h"
#include "Async/Async.h"
#include "Engine/Polys.h"
#include "MeshUtilitiesCommon.h"
#include "OverlappingCorners.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

namespace DatasmithRuntime
{
	extern const FString TexturePrefix;
	extern const FString MaterialPrefix;
	extern const FString MeshPrefix;

	bool /*FDatasmithStaticMeshImporter::*/ShouldRecomputeNormals(const FMeshDescription& MeshDescription, int32 BuildRequirements)
	{
		const TVertexInstanceAttributesConstRef<FVector3f> Normals = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
		check(Normals.IsValid());
		return Algo::AnyOf(MeshDescription.VertexInstances().GetElementIDs(), [&](const FVertexInstanceID& InstanceID) { return !Normals[InstanceID].IsNormalized(); });
	}

	bool /*FDatasmithStaticMeshImporter::*/ShouldRecomputeTangents(const FMeshDescription& MeshDescription, int32 BuildRequirements)
	{
		const TVertexInstanceAttributesConstRef<FVector3f> Tangents = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
		check(Tangents.IsValid());
		return Algo::AnyOf(MeshDescription.VertexInstances().GetElementIDs(), [&](const FVertexInstanceID& InstanceID) { return !Tangents[InstanceID].IsNormalized(); });
	}

	int32 GetNextOpenUVChannel(FMeshDescription& MeshDescription)
	{
		FStaticMeshConstAttributes Attributes(MeshDescription);
		int32 NumberOfUVs = Attributes.GetVertexInstanceUVs().GetNumChannels();
		int32 FirstEmptyUVs = 0;

		for (; FirstEmptyUVs < NumberOfUVs; ++FirstEmptyUVs)
		{
			const TVertexInstanceAttributesConstRef<FVector2f> UVChannels = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
			const FVector2f DefValue = UVChannels.GetDefaultValue();
			bool bHasNonDefaultValue = false;

			for (FVertexInstanceID InstanceID : MeshDescription.VertexInstances().GetElementIDs())
			{
				if (UVChannels.Get(InstanceID, FirstEmptyUVs) != DefValue)
				{
					bHasNonDefaultValue = true;
					break;
				}
			}

			if (!bHasNonDefaultValue)
			{
				//We found an "empty" channel.
				break;
			}
		}

		return FirstEmptyUVs < MAX_MESH_TEXTURE_COORDS_MD ? FirstEmptyUVs : -1;
	}

	float Get2DSurface(const FVector4& Dimensions)
	{
		if (Dimensions[0] >= Dimensions[1] && Dimensions[2] >= Dimensions[1])
		{
			return Dimensions[0] * Dimensions[2];
		}
		if (Dimensions[0] >= Dimensions[2] && Dimensions[1] >= Dimensions[2])
		{
			return Dimensions[0] * Dimensions[1];
		}

		return Dimensions[2] * Dimensions[1];
	}

	float CalcBlendWeight(const FVector4& Dimensions, float MaxArea, float Max2DSurface)
	{
		const float Current2DSurface = Get2DSurface(Dimensions);
		const float Weight = FMath::Sqrt((Dimensions[3] / MaxArea)) + FMath::Sqrt(Current2DSurface / Max2DSurface);

		return Weight;
	}

	void CalculateMeshesLightmapWeights(const TArray< FSceneGraphId >& MeshElementArray, const TMap< FSceneGraphId, TSharedPtr< IDatasmithElement > >& Elements, TMap< FSceneGraphId, float >& LightmapWeights)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::CalculateMeshesLightmapWeights);

		LightmapWeights.Reserve(MeshElementArray.Num());

		float MaxArea = 0.0f;
		float Max2DSurface = 0.0f;

		// Compute the max values based on all meshes in the Datasmith Scene

		for (FSceneGraphId MeshElementId : MeshElementArray)
		{
			TSharedPtr< IDatasmithMeshElement > MeshElement = StaticCastSharedPtr< IDatasmithMeshElement >(Elements[MeshElementId]);

			MaxArea = FMath::Max(MaxArea, MeshElement->GetArea());

			const FVector4 Dimensions(MeshElement->GetWidth(), MeshElement->GetDepth(), MeshElement->GetHeight(), MeshElement->GetArea());

			Max2DSurface = FMath::Max(Max2DSurface, Get2DSurface(Dimensions));
		}

		float MaxWeight = 0.0f;

		for (FSceneGraphId MeshElementId : MeshElementArray)
		{
			TSharedPtr< IDatasmithMeshElement > MeshElement = StaticCastSharedPtr< IDatasmithMeshElement >(Elements[MeshElementId]);

			const FVector4 Dimensions(MeshElement->GetWidth(), MeshElement->GetDepth(), MeshElement->GetHeight(), MeshElement->GetArea());

			const float MeshWeight = CalcBlendWeight(Dimensions, MaxArea, Max2DSurface);

			MaxWeight = FMath::Max(MaxWeight, MeshWeight);

			LightmapWeights.Add(MeshElementId, MeshWeight);
		}

		for (FSceneGraphId MeshElementId : MeshElementArray)
		{
			LightmapWeights[MeshElementId] /= MaxWeight;
		}
	}

	int32 GenerateLightmapUVResolution(FMeshDescription& Mesh, int32 SrcLightmapIndex, int32 MinLightmapResolution)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::GenerateLightmapUVResolution);

		// Determine the absolute minimum lightmap resolution that can be used for packing
		FOverlappingCorners OverlappingCorners;
		FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, Mesh, THRESH_POINTS_ARE_SAME);

		// Packing expects at least one texel per chart. This is the absolute minimum to generate valid UVs.
		float ChartCount = FStaticMeshOperations::GetUVChartCount(Mesh, SrcLightmapIndex, ELightmapUVVersion::Latest, OverlappingCorners);
		const int32 AbsoluteMinResolution = 1 << FMath::CeilLogTwo(FMath::Sqrt(ChartCount));

		return FMath::Clamp(MinLightmapResolution, AbsoluteMinResolution, 512);
	}

	void ProcessCollision(UStaticMesh* StaticMesh, FDatasmithMeshElementPayload& Payload)
	{
		// The following code is copied from StaticMeshEdit AddConvexGeomFromVertices (inaccessible outside UnrealEd)
		if (!StaticMesh)
		{
			return;
		}

		TArray< FVector3f > VertexPositions;
		DatasmithMeshHelper::ExtractVertexPositions(Payload.CollisionMesh, VertexPositions);
		if (VertexPositions.Num() == 0)
		{
			VertexPositions = MoveTemp( Payload.CollisionPointCloud );
		}

		if (VertexPositions.Num() > 0)
		{
#if WITH_EDITORONLY_DATA
			StaticMesh->SetCustomizedCollision(true);
#endif
			if (!ensure(StaticMesh->GetBodySetup()))
			{
				return;
			}

			// Convex elements must be removed first since the re-import process uses the same flow
			FKAggregateGeom& AggGeom = StaticMesh->GetBodySetup()->AggGeom;
			AggGeom.ConvexElems.Reset();
			FKConvexElem& ConvexElem = AggGeom.ConvexElems.AddDefaulted_GetRef();

			ConvexElem.VertexData.Reserve(VertexPositions.Num());
			for (const FVector3f& Position : VertexPositions)
			{
				ConvexElem.VertexData.Add(FVector(Position));
			}

			ConvexElem.UpdateElemBox();
		}
	}

	TMap<uint32, TStrongObjectPtr<UObject>> FAssetRegistry::RegistrationMap;
	TMap<uint32, TMap<FSceneGraphId,FAssetData>*> FAssetRegistry::SceneMappings;

	union FRegistryKey
	{
		uint32 Pair[2];
		uint64 Value;

		FRegistryKey(uint64 InValue) { Value = InValue; }
		FRegistryKey(uint32 SceneKey, FSceneGraphId AssetId) { Pair[0] = SceneKey; Pair[1] = AssetId; }
	};

	void FAssetRegistry::RegisterMapping(uint32 SceneKey, TMap<FSceneGraphId,FAssetData>* AssetsMapping)
	{
		SceneMappings.Add(SceneKey, AssetsMapping);
	}

	void FAssetRegistry::UnregisterMapping(uint32 SceneKey)
	{
		ensure(SceneMappings.Contains(SceneKey));

		SceneMappings.Remove(SceneKey);
	}

	void FAssetRegistry::RegisterAssetData(UObject* Asset, uint32 SceneKey, FAssetData& AssetData)
	{
		check(IsInGameThread());

		ensure(SceneMappings.Contains(SceneKey));

		if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Asset))
		{
			UDatasmithRuntimeAuxiliaryData* AuxillaryData = AssetUserData->GetAssetUserData<UDatasmithRuntimeAuxiliaryData>();

			if(AuxillaryData == nullptr)
			{
				AuxillaryData = NewObject<UDatasmithRuntimeAuxiliaryData>(Asset, NAME_None, RF_NoFlags);
				AssetUserData->AddAssetUserData(AuxillaryData);
			}

			if (AuxillaryData->Referencers.Num() == 0 && !RegistrationMap.Contains(AssetData.Hash))
			{
				RegistrationMap.Emplace(AssetData.Hash, Asset);
			}

			FRegistryKey RegistryKey(SceneKey, AssetData.ElementId);

			AuxillaryData->Referencers.Add(RegistryKey.Value);

			if (AuxillaryData->bIsCompleted)
			{
				AssetData.AddState(EAssetState::Completed);
			}
			else
			{
				AssetData.ClearState(EAssetState::Completed);
			}
		}
	}

	int32 FAssetRegistry::UnregisterAssetData(UObject* Asset, uint32 SceneKey, FSceneGraphId AssetId)
	{
		check(IsInGameThread());

		ensure(SceneMappings.Contains(SceneKey));

		if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Asset))
		{
			if (UDatasmithRuntimeAuxiliaryData* AuxillaryData = AssetUserData->GetAssetUserData<UDatasmithRuntimeAuxiliaryData>())
			{
				FRegistryKey RegistryKey(SceneKey, AssetId);

				if (AuxillaryData->Referencers.Contains(RegistryKey.Value))
				{
					AuxillaryData->Referencers.Remove(RegistryKey.Value);

					return AuxillaryData->Referencers.Num();
				}
			}
		}

		ensure(false);
		return -1;
	}

	void FAssetRegistry::SetObjectCompletion(UObject* Asset, bool bIsCompleted)
	{
		if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Asset))
		{
			if (UDatasmithRuntimeAuxiliaryData* AuxillaryData = AssetUserData->GetAssetUserData<UDatasmithRuntimeAuxiliaryData>())
			{
				AuxillaryData->bIsCompleted.store(bIsCompleted);

				if (bIsCompleted)
				{
					for (uint64 ReferencerKey : AuxillaryData->Referencers)
					{
						const FRegistryKey RegistryKey(ReferencerKey);

						ensure(SceneMappings.Contains(RegistryKey.Pair[0]));
						TMap<FSceneGraphId, FAssetData>& AssetsMapping = *(SceneMappings[RegistryKey.Pair[0]]);

						ensure(AssetsMapping.Contains(RegistryKey.Pair[1]));
						AssetsMapping[RegistryKey.Pair[1]].AddState(EAssetState::Completed);
					}
				}
				else
				{
					for (uint64 ReferencerKey : AuxillaryData->Referencers)
					{
						const FRegistryKey RegistryKey(ReferencerKey);

						ensure(SceneMappings.Contains(RegistryKey.Pair[0]));
						TMap<FSceneGraphId, FAssetData>& AssetsMapping = *(SceneMappings[RegistryKey.Pair[0]]);

						ensure(AssetsMapping.Contains(RegistryKey.Pair[1]));
						AssetsMapping[RegistryKey.Pair[1]].ClearState(EAssetState::Completed);
					}
				}

				return;
			}
		}

		ensure(false);
	}

	bool FAssetRegistry::IsObjectCompleted(UObject* Asset)
	{
		bool bIsCompleted = false;

		if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Asset))
		{
			if (UDatasmithRuntimeAuxiliaryData* AuxillaryData = AssetUserData->GetAssetUserData<UDatasmithRuntimeAuxiliaryData>())
			{
				for (uint64 ReferencerKey : AuxillaryData->Referencers)
				{
					const FRegistryKey RegistryKey(ReferencerKey);

					ensure(SceneMappings.Contains(RegistryKey.Pair[0]));
					TMap<FSceneGraphId, FAssetData>& AssetsMapping = *(SceneMappings[RegistryKey.Pair[0]]);

					ensure(AssetsMapping.Contains(RegistryKey.Pair[1]));
					bIsCompleted |= AssetsMapping[RegistryKey.Pair[1]].HasState(EAssetState::Completed);
				}

				return bIsCompleted;
			}
		}

		return bIsCompleted;
	}

	int32 FAssetRegistry::GetAssetReferenceCount(UObject * Asset)
	{
		if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Asset))
		{
			if (UDatasmithRuntimeAuxiliaryData* AuxillaryData = AssetUserData->GetAssetUserData<UDatasmithRuntimeAuxiliaryData>())
			{
				return AuxillaryData->Referencers.Num();
			}
		}

		return -1;
	}

	void FAssetRegistry::UnregisteredAssetsData(UObject* Asset, uint32 SceneKey, TFunction<void(FAssetData& AssetData)> UpdateFunc)
	{
		if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Asset))
		{
			if (UDatasmithRuntimeAuxiliaryData* AuxillaryData = AssetUserData->GetAssetUserData<UDatasmithRuntimeAuxiliaryData>())
			{
				TArray<uint64> ReferencersToDelete;

				for (uint64 ReferencerKey : AuxillaryData->Referencers)
				{
					const FRegistryKey RegistryKey(ReferencerKey);

					// If SceneKey is specified, only apply function to assets of that scene
					if (SceneKey && SceneKey != RegistryKey.Pair[0])
					{
						continue;
					}

					ReferencersToDelete.Add(ReferencerKey);

				}

				for (uint64 ReferencerKey : ReferencersToDelete)
				{
					AuxillaryData->Referencers.Remove(ReferencerKey);
				}

				for (uint64 ReferencerKey : ReferencersToDelete)
				{
					const FRegistryKey RegistryKey(ReferencerKey);

					ensure(SceneMappings.Contains(RegistryKey.Pair[0]));
					TMap<FSceneGraphId, FAssetData>& AssetsMapping = *(SceneMappings[RegistryKey.Pair[0]]);

					ensure(AssetsMapping.Contains(RegistryKey.Pair[1]));
					UpdateFunc(AssetsMapping[RegistryKey.Pair[1]]);
				}

				return;
			}
		}

		ensure(false);
	}

	UObject* FAssetRegistry::FindObjectFromHash(uint32 ElementHash)
	{
		TStrongObjectPtr<UObject>* AssetPtr = RegistrationMap.Find(ElementHash);
		return AssetPtr ? (*AssetPtr).Get() : nullptr;
	}

	bool FAssetRegistry::CleanUp()
	{
		TArray<uint32> EntriesToDelete;
		EntriesToDelete.Reserve(RegistrationMap.Num());

		for (TPair<uint32, TStrongObjectPtr<UObject>>& Entry : RegistrationMap)
		{
			if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Entry.Value.Get()))
			{
				if (UDatasmithRuntimeAuxiliaryData* AuxillaryData = AssetUserData->GetAssetUserData<UDatasmithRuntimeAuxiliaryData>())
				{
					if (AuxillaryData->Referencers.Num() == 0)
					{
						Entry.Value->ClearFlags(RF_Public);
						Entry.Value->SetFlags(RF_Transient);
						Entry.Value->Rename(nullptr, nullptr, REN_NonTransactional | REN_DontCreateRedirectors);
						Entry.Value->MarkAsGarbage();
						Entry.Value.Reset();

						EntriesToDelete.Add(Entry.Key);
					}
				}
			}
		}

		for (uint32 ElementHash : EntriesToDelete)
		{
			RegistrationMap.Remove(ElementHash);
		}

		return EntriesToDelete.Num() > 0;
	}

	// Code below has been borrowed from GenerateKDopAsSimpleCollision in GeomFitUtils.cpp
	// to generate k-DOP (k-Discrete Oriented Polytopes) with 26 polytopes
	#define RCP_SQRT2 (0.70710678118654752440084436210485f)
	#define RCP_SQRT3 (0.57735026918962576450914878050196f)

	const FVector KDopDir26[26] =
	{
		FVector( 1.f, 0.f, 0.f),
		FVector(-1.f, 0.f, 0.f),
		FVector( 0.f, 1.f, 0.f),
		FVector( 0.f,-1.f, 0.f),
		FVector( 0.f, 0.f, 1.f),
		FVector( 0.f, 0.f,-1.f),
		FVector( 0.f, RCP_SQRT2,  RCP_SQRT2),
		FVector( 0.f,-RCP_SQRT2, -RCP_SQRT2),
		FVector( 0.f, RCP_SQRT2, -RCP_SQRT2),
		FVector( 0.f,-RCP_SQRT2,  RCP_SQRT2),
		FVector( RCP_SQRT2, 0.f,  RCP_SQRT2),
		FVector(-RCP_SQRT2, 0.f, -RCP_SQRT2),
		FVector( RCP_SQRT2, 0.f, -RCP_SQRT2),
		FVector(-RCP_SQRT2, 0.f,  RCP_SQRT2),
		FVector( RCP_SQRT2,  RCP_SQRT2, 0.f),
		FVector(-RCP_SQRT2, -RCP_SQRT2, 0.f),
		FVector( RCP_SQRT2, -RCP_SQRT2, 0.f),
		FVector(-RCP_SQRT2,  RCP_SQRT2, 0.f),
		FVector( RCP_SQRT3,  RCP_SQRT3,  RCP_SQRT3),
		FVector( RCP_SQRT3,  RCP_SQRT3, -RCP_SQRT3),
		FVector( RCP_SQRT3, -RCP_SQRT3,  RCP_SQRT3),
		FVector( RCP_SQRT3, -RCP_SQRT3, -RCP_SQRT3),
		FVector(-RCP_SQRT3,  RCP_SQRT3,  RCP_SQRT3),
		FVector(-RCP_SQRT3,  RCP_SQRT3, -RCP_SQRT3),
		FVector(-RCP_SQRT3, -RCP_SQRT3,  RCP_SQRT3),
		FVector(-RCP_SQRT3, -RCP_SQRT3, -RCP_SQRT3),
	};

	constexpr float HalfWorldMax = HALF_WORLD_MAX;

	void GenerateKDopAsSimpleCollision(UBodySetup* BodySetup, const FStaticMeshLODResources& Resources, const FVector* Directions, int32 DirectionCount)
	{
		TArray<float> MaxDistances;
		MaxDistances.Reserve(DirectionCount);

		for (int32 Index = 0; Index < DirectionCount; ++Index)
		{
			MaxDistances.Add(-MAX_FLT);
		}

		// For each vertex, project along each kdop direction, to find the max in that direction.
		const FPositionVertexBuffer& PositionVertexBuffer = Resources.VertexBuffers.PositionVertexBuffer;
		for(int32 Index = 0; Index < Resources.GetNumVertices(); ++Index)
		{
			for(int32 DirIndex = 0; DirIndex < DirectionCount; ++DirIndex)
			{
				const float Dist = (FVector)PositionVertexBuffer.VertexPosition(Index) | Directions[DirIndex];
				MaxDistances[DirIndex] = FMath::Max(Dist, MaxDistances[DirIndex]);
			}
		}

		// Inflate MaxDistances to ensure it is no degenerate
		const float MinSize = 0.1f;
		for (int32 Index = 0; Index < DirectionCount; ++Index)
		{
			MaxDistances[Index] += MinSize;
		}

		// Now we have the Planes of the kdop, we work out the face polygons.
		TArray<FPlane> Planes;
		Planes.Reserve(DirectionCount);
		for (int32 Index = 0; Index < DirectionCount; ++Index)
		{
			Planes.Add( FPlane(Directions[Index], MaxDistances[Index]) );
		}

		TArray<FPoly> Element;
		Element.Reserve(DirectionCount);
		for (int32 Index = 0; Index < DirectionCount; ++Index)
		{
			FPoly&	Polygon = Element.AddZeroed_GetRef();
			FVector3f Base, AxisX, AxisY;

			Polygon.Init();
			Polygon.Normal = (FVector3f)Planes[Index];
			Polygon.Normal.FindBestAxisVectors(AxisX, AxisY);

			Base = FVector3f(Planes[Index] * Planes[Index].W);

			Polygon.Vertices.Reserve(4);
			new(Polygon.Vertices) FVector3f(Base + AxisX * HalfWorldMax + AxisY * HalfWorldMax);
			new(Polygon.Vertices) FVector3f(Base + AxisX * HalfWorldMax - AxisY * HalfWorldMax);
			new(Polygon.Vertices) FVector3f(Base - AxisX * HalfWorldMax - AxisY * HalfWorldMax);
			new(Polygon.Vertices) FVector3f(Base - AxisX * HalfWorldMax + AxisY * HalfWorldMax);

			for (int32 Jndex = 0; Jndex < DirectionCount; ++Jndex)
			{
				if(Index != Jndex)
				{
					if(!Polygon.Split(-FVector3f(Planes[Jndex]), FVector3f(Planes[Jndex] * Planes[Jndex].W)))
					{
						Polygon.Vertices.Empty();
						break;
					}
				}
			}

			if(Polygon.Vertices.Num() < 3)
			{
				// If poly resulted in no verts, remove from array
				Element.Pop(EAllowShrinking::No);
			}
			else
			{
				// Other stuff...
				Polygon.iLink = Index;
				Polygon.CalcNormal(1);
			}
		}

		if(Element.Num() < 4)
		{
			return;
		}

		FKConvexElem& ConvexElem = BodySetup->AggGeom.ConvexElems.AddDefaulted_GetRef();

		for (FPoly& Poly : Element)
		{
			for (FVector3f& Position : Poly.Vertices)
			{
				ConvexElem.VertexData.Add((FVector)Position);
			}
		}

		ConvexElem.UpdateElemBox();
	}
	// End of Code borrowed from GenerateKDopAsSimpleCollision in GeomFitUtils.cpp

	void BuildCollision(UBodySetup* BodySetup, ECollisionTraceFlag CollisionFlag, const FStaticMeshLODResources& Resources)
	{
		BodySetup->CollisionTraceFlag = CollisionFlag;

		if (BodySetup->CollisionTraceFlag == ECollisionTraceFlag::CTF_UseDefault)
		{
			BodySetup->CollisionTraceFlag = UPhysicsSettings::Get()->DefaultShapeComplexity;
		}

		// Use k-DOP 26 as the collision mesh if simple collision is required
		// #ue_dsruntime - TODO | Choose better shape of collision mesh
		if (BodySetup->CollisionTraceFlag != ECollisionTraceFlag::CTF_UseComplexAsSimple && BodySetup->AggGeom.ConvexElems.Num() == 0)
		{
			GenerateKDopAsSimpleCollision(BodySetup, Resources, KDopDir26, 26);
		}

		// Creation of collision meshes can only happen on game thread
		AsyncTask(ENamedThreads::GameThread, [BodySetup] {
#ifdef LIVEUPDATE_TIME_LOGGING
			double LocalStartTime = FPlatformTime::Seconds();
			Timer __Timer(LocalStartTime, "BuildCollision");
#endif
			BodySetup->CreatePhysicsMeshes();
		});
	}
} // End of namespace DatasmithRuntime
