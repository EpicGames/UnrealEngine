// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Async/TaskGraphInterfaces.h"
#include "SceneImporter.h"


#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSourceData.h"
#include "Engine/World.h"

#include "UObject/StrongObjectPtr.h"
#include "DatasmithRuntimeUtils.generated.h"

enum ECollisionTraceFlag : int;
enum class EDatasmithTextureFormat : uint8;

class IDatasmithBaseMaterialElement;
class IDatasmithElement;
class IDatasmithMeshElement;
class IDatasmithTextureElement;
class IDatasmithUEPbrMaterialElement;
class FSceneImporter;
class UBodySetup;
class UClass;
class UMaterial;
class UMaterialInstanceDynamic;
class USceneComponent;
class UWorld;

struct FTextureData;
struct FActorData;
struct FDatasmithMeshElementPayload;
struct FMeshDescription;
struct FStaticMeshLODResources;

// Class deriving from UStaticMesh to allow the cooking of collision meshes at runtime
// To do so, bAllowCPUAccess must be true AND  the metod GetWorld() must return a valid world
UCLASS()
class URuntimeMesh : public UStaticMesh
{
	GENERATED_BODY()

public:
	URuntimeMesh()
		: World(nullptr)
	{
		// Set bAllowCPUAccess to true to allow the copy of render data triangles to the collision mesh
		bAllowCPUAccess = true;
	}

	// UObject overrides
	// Overridden to allow cooking of collision meshes, simple and complex, from static mesh at runtime
	virtual UWorld* GetWorld() const override  { return World ? World : UStaticMesh::GetWorld(); }
	// End UObject overrides

	// Use valid world to allow cooking of collision meshes, simple and complex, from static mesh at runtime 
	void SetWorld(UWorld* InWorld) { World = InWorld; }

private:
	UWorld* World;
};

namespace DatasmithRuntime
{
	enum class EDSResizeTextureMode
	{
		NoResize,
		NearestPowerOfTwo,
		PreviousPowerOfTwo,
		NextPowerOfTwo
	};

	FORCEINLINE uint32 GetTypeHash(const FMD5Hash& Hash, EDataType Type = EDataType::None)
	{
		return HashCombine(FCrc::MemCrc32(Hash.GetBytes(),Hash.GetSize()), 1u << (uint8)Type);
	}

	extern void CalculateMeshesLightmapWeights(const TArray< FSceneGraphId >& MeshElementArray, const TMap< FSceneGraphId, TSharedPtr< IDatasmithElement > >& Elements, TMap< FSceneGraphId, float >& LightmapWeights);

	// Borrowed from UVGenerationUtils::GetNextOpenUVChannel(UStaticMesh* StaticMesh, int32 LODIndex)
	extern int32 GetNextOpenUVChannel(FMeshDescription& MeshDescription);

	// Borrowed from  UVGenerationUtils::SetupGeneratedLightmapUVResolution(UStaticMesh* StaticMesh, int32 LODIndex)
	extern int32 GenerateLightmapUVResolution(FMeshDescription& Mesh, int32 SrcLightmapIndex, int32 MinLightmapResolution);

	// Borrowed from FDatasmithStaticMeshImporter::ProcessCollision(UStaticMesh* StaticMesh, const TArray< FVector >& VertexPositions)
	extern void ProcessCollision(UStaticMesh* StaticMesh, FDatasmithMeshElementPayload& Payload);

	extern void BuildCollision(UBodySetup* Body, ECollisionTraceFlag CollisionFlag, const FStaticMeshLODResources& Resources);


	extern bool /*FDatasmithStaticMeshImporter::*/ShouldRecomputeNormals(const FMeshDescription& MeshDescription, int32 BuildRequirements);

	extern bool /*FDatasmithStaticMeshImporter::*/ShouldRecomputeTangents(const FMeshDescription& MeshDescription, int32 BuildRequirements);

	extern void ImageReaderInitialize();

	extern bool GetTextureDataFromFile(const TCHAR* Filename, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData);
	extern bool GetTextureDataFromBuffer(TArray<uint8>& Bytes, EDatasmithTextureFormat Format, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData);

	class FAssetRegistry
	{
	public:
		static void RegisterMapping(uint32 SceneKey, TMap<FSceneGraphId, FAssetData>* AssetsMapping);

		static void UnregisterMapping(uint32 SceneKey);

		static void RegisterAssetData(UObject* Asset, uint32 SceneKey, FAssetData& AssetData);

		static int32 UnregisterAssetData(UObject* Asset, uint32 SceneKey, FSceneGraphId AssetId);

		static void UnregisteredAssetsData(UObject* Asset, uint32 SceneKey, TFunction<void(FAssetData& AssetData)> UpdateFunc);

		static void SetObjectCompletion(UObject* Asset, bool bIsCompleted);

		static bool IsObjectCompleted(UObject* Asset);

		static UObject* FindObjectFromHash(uint32 ElementHash);

		static int32 GetAssetReferenceCount(UObject* Asset);

		/** @return true if some assets have been marked for deletion */
		static bool CleanUp();

	private:
		static TMap<uint32, TStrongObjectPtr<UObject>> RegistrationMap;
		static TMap<uint32, TMap<FSceneGraphId,FAssetData>*> SceneMappings;
	};

	extern const FName RuntimeTag;

	extern void RenameObject(UObject* Object, const TCHAR* DesiredName, UObject* NewOwner = nullptr);

	extern USceneComponent* CreateComponent(FActorData& ActorData, UClass* Class, AActor* Owner);

	template<typename T>
	T* CreateComponent(FActorData& ActorData, AActor* Owner)
	{
		return Cast<T>(CreateComponent(ActorData, T::StaticClass(), Owner));
	}

	template<typename T>
	T* CreateActor(UWorld* World)
	{
		T* NewActor = Cast<T>(World->SpawnActor(T::StaticClass(), nullptr, nullptr));
		check(NewActor);

		return NewActor;
	}
}
