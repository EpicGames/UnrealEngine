// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectLinkUtils.h"

#include "DatasmithRuntime.h"
#include "DatasmithRuntimeBlueprintLibrary.h"

#include "DatasmithCore.h"
#include "DirectLinkLog.h"

#include "Misc/ScopeRWLock.h"
#include "Serialization/MemoryWriter.h"
#include "Tickable.h"
#include "UObject/StrongObjectPtr.h"

#if !WITH_EDITOR
#include "Modules/ModuleManager.h"
#endif

const TCHAR* EndPointName = TEXT("DatasmithRuntime");

namespace DatasmithRuntime
{
	// UDirectLinkProxy object used by the game's UI
	static TStrongObjectPtr<UDirectLinkProxy> DirectLinkProxy;

	UDirectLinkProxy* GetDirectLinkProxy()
	{
		return DirectLinkProxy.Get();
	}

	// Helper class to expose some functionalities of the DirectLink end point
	// This is a tickable object in order to update the Game on changes happening on the DirectLink network
	class FDirectLinkEndpointProxy : public DirectLink::IEndpointObserver, public FTickableGameObject
	{
	public:
		virtual ~FDirectLinkEndpointProxy();

		// Begin DirectLink::IEndpointObserver interface
		void OnStateChanged(const DirectLink::FRawInfo& RawInfo) override;
		// End DirectLink::IEndpointObserver interface

		/** Register a destination to the end point */
		bool RegisterDestination(const TCHAR* StreamName, TSharedPtr<FDestinationProxy> DestinationProxy);

		/** Unregister a destination to the end point */
		void UnregisterDestination(TSharedPtr<FDestinationProxy> DestinationProxy);

		/** Open a connection between a given source and a given destination */
		bool OpenConnection(const DirectLink::FSourceHandle& SourceHandle, const DirectLink::FDestinationHandle& DestinationHandle);

		/** Close a connection between a given source and a given destination */
		void CloseConnection(const DirectLink::FSourceHandle& SourceHandle, const DirectLink::FDestinationHandle& DestinationHandle);

		/** Returns a source handle based on a given UI identifier */
		DirectLink::FSourceHandle GetSourceHandleFromIdentifier(uint32 SourceIdentifier);

		/** Compute a UI identifier for a given source handle */
		uint32 ComputeSourceIdentifier(const DirectLink::FSourceHandle& SourceHandle);

		/** Returns the name of the source associated to a given handle */
		FString GetSourceName(const DirectLink::FSourceHandle& SourceHandle);

		/** Returns the source handle a give destination is connected to */
		DirectLink::FSourceHandle GetConnection(const DirectLink::FDestinationHandle& DestinationHandle);

		/** Set the delegate which will be called when there are changes on the DirectLink network */
		void SetChangeNotifier(FDatasmithRuntimeChangeEvent* InNotifyChange)
		{
			NotifyChange = InNotifyChange;
		}

		/** Returns the name of the end point */
		FString GetEndPointName();

		/** Returns the list of public sources available on the DirectLink network */
		const TArray<FDatasmithRuntimeSourceInfo>& GetListOfSources() const;

	protected:
		//~ Begin FTickableEditorObject interface
		virtual void Tick(float DeltaSeconds) override;

		virtual bool IsTickable() const override
		{
			return ReceiverEndpoint.IsValid() && bIsDirty;
		}

		virtual TStatId GetStatId() const override
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FDirectLinkEndpointProxy, STATGROUP_Tickables);
		}
		//~ End FTickableEditorObject interface

	private:
		FDirectLinkEndpointProxy() : bIsDirty(false), NotifyChange(nullptr) {}

	private:
		/** Unique pointer to the DirectLink endpoint */
		TUniquePtr<DirectLink::FEndpoint> ReceiverEndpoint;

		/** Set of destinations registered to the DirectLink end point */
		TMap<DirectLink::FDestinationHandle, TSharedPtr<FDestinationProxy>> DestinationList;

		/**
		 * Lock used when processing the FRawInfo provided by the DirectLink end point
		 * The calls to Tick and OnStateChanged are happening on different threads.
		 */
		mutable FRWLock RawInfoCopyLock;

		/** Hash of the last FRawInfo processed */
		FMD5Hash LastHash;

		TArray<FDatasmithRuntimeSourceInfo> LastSources;

		std::atomic_bool bIsDirty;

		/** Pointer to the delegate to call when there are changes on the DirectLink network */
		FDatasmithRuntimeChangeEvent* NotifyChange;

		friend FDestinationProxy;
		friend UDirectLinkProxy;
	};
}

UDirectLinkProxy::UDirectLinkProxy()
{
	using namespace DatasmithRuntime;

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FDestinationProxy::GetEndpointProxy().SetChangeNotifier(&OnDirectLinkChange);
	}
}

UDirectLinkProxy::~UDirectLinkProxy()
{
	using namespace DatasmithRuntime;

	FDestinationProxy::GetEndpointProxy().SetChangeNotifier(nullptr);
}

FString UDirectLinkProxy::GetEndPointName()
{
	return EndPointName;
}

TArray<FDatasmithRuntimeSourceInfo> UDirectLinkProxy::GetListOfSources()
{
	using namespace DatasmithRuntime;

	return FDestinationProxy::GetEndpointProxy().GetListOfSources();
}

namespace DatasmithRuntime
{
	TSharedPtr<FDirectLinkEndpointProxy> FDestinationProxy::EndpointProxy;

	void FDestinationProxy::InitializeEndpointProxy()
	{
		ensure(!EndpointProxy.IsValid());
		EndpointProxy = TSharedPtr<FDirectLinkEndpointProxy>(new FDirectLinkEndpointProxy());
		ensure(EndpointProxy.IsValid());

		// Create associated UObject to access DirectLink features from the BP UI
		DirectLinkProxy = TStrongObjectPtr<UDirectLinkProxy>(NewObject<UDirectLinkProxy>());


#if !NO_LOGGING
		LogDatasmith.SetVerbosity( ELogVerbosity::Warning );
#ifndef DIRECTLINK_LOG
		LogDirectLink.SetVerbosity( ELogVerbosity::Warning );
		LogDirectLinkNet.SetVerbosity( ELogVerbosity::Warning );
#endif
#if !WITH_EDITOR || defined(NO_DL_DEBUG)
		LogDirectLink.SetVerbosity( ELogVerbosity::Warning );
		LogDirectLinkNet.SetVerbosity( ELogVerbosity::Warning );
#endif
#endif
	}

	void FDestinationProxy::ShutdownEndpointProxy()
	{
		DirectLinkProxy.Reset();
		FDestinationProxy::EndpointProxy.Reset();
	}

	const TArray<FDatasmithRuntimeSourceInfo>& FDestinationProxy::GetListOfSources()
	{
		static TArray<FDatasmithRuntimeSourceInfo> EmptyList;

		return FDestinationProxy::EndpointProxy ? FDestinationProxy::EndpointProxy->GetListOfSources() : EmptyList;
	}

	FDirectLinkEndpointProxy::~FDirectLinkEndpointProxy()
	{
		if (ReceiverEndpoint.IsValid())
		{
			ReceiverEndpoint->RemoveEndpointObserver(this);
			ReceiverEndpoint.Reset();
		}
	}

	bool FDirectLinkEndpointProxy::RegisterDestination(const TCHAR* StreamName, TSharedPtr<FDestinationProxy> DestinationProxy)
	{
		using namespace DirectLink;

		if (DestinationProxy.IsValid())
		{
			if (!ReceiverEndpoint.IsValid())
			{
				ReceiverEndpoint.Reset();
				bool bInit = true;
	#if !WITH_EDITOR
				bInit = (FModuleManager::Get().LoadModule(TEXT("Messaging")))
					&& (FModuleManager::Get().LoadModule(TEXT("Networking")))
					&& (FModuleManager::Get().LoadModule(TEXT("UdpMessaging")));
	#endif

				if (!bInit)
				{
					return false;
				}

				ReceiverEndpoint = MakeUnique<FEndpoint>(EndPointName);
				ReceiverEndpoint->AddEndpointObserver(this);
				ReceiverEndpoint->SetVerbose();
			}

			check(ReceiverEndpoint.IsValid());

			DestinationProxy->GetDestinationHandle() = ReceiverEndpoint->AddDestination(StreamName, DirectLink::EVisibility::Public, StaticCastSharedPtr<IConnectionRequestHandler>(DestinationProxy));

			if (DestinationProxy->GetDestinationHandle().IsValid())
			{
				DestinationList.Add(DestinationProxy->GetDestinationHandle(), DestinationProxy);
				return true;
			}
		}

		return false;
	}

	void FDirectLinkEndpointProxy::UnregisterDestination(TSharedPtr<FDestinationProxy> DestinationProxy)
	{
		DirectLink::FDestinationHandle& DestinationHandle = DestinationProxy->GetDestinationHandle();

		if (ReceiverEndpoint.IsValid() && DestinationList.Contains(DestinationHandle))
		{
			DestinationList.Remove(DestinationHandle);
			ReceiverEndpoint->RemoveDestination(DestinationHandle);
		}
	}

	bool FDirectLinkEndpointProxy::OpenConnection(const DirectLink::FSourceHandle& SourceHandle, const DirectLink::FDestinationHandle& DestinationHandle)
	{
		using namespace DirectLink;

		if (ReceiverEndpoint.IsValid())
		{
			FEndpoint::EOpenStreamResult Result = ReceiverEndpoint->OpenStream(SourceHandle, DestinationHandle);

			return Result == FEndpoint::EOpenStreamResult::Opened || Result == FEndpoint::EOpenStreamResult::AlreadyOpened;
		}

		return false;
	}

	void FDirectLinkEndpointProxy::CloseConnection(const DirectLink::FSourceHandle& SourceHandle, const DirectLink::FDestinationHandle& DestinationHandle)
	{
		if (ReceiverEndpoint.IsValid())
		{
			ReceiverEndpoint->CloseStream(SourceHandle, DestinationHandle);
		}

	}

	FString FDirectLinkEndpointProxy::GetSourceName(const DirectLink::FSourceHandle& SourceHandle)
	{
		using namespace DirectLink;

		if (ReceiverEndpoint.IsValid() && SourceHandle.IsValid())
		{
			FRawInfo RawInfo = ReceiverEndpoint->GetRawInfoCopy();

			if (FRawInfo::FDataPointInfo* DataPointInfoPtr = RawInfo.DataPointsInfo.Find(SourceHandle))
			{
				if (RawInfo.EndpointsInfo.Contains(DataPointInfoPtr->EndpointAddress))
				{
					const FRawInfo::FEndpointInfo& EndPointInfo = RawInfo.EndpointsInfo[DataPointInfoPtr->EndpointAddress];

					return DataPointInfoPtr->Name + TEXT("-") + EndPointInfo.ExecutableName + TEXT("-") + FString::FromInt((int32)EndPointInfo.ProcessId);
				}
			}
		}

		return FString();
	}

	uint32 FDirectLinkEndpointProxy::ComputeSourceIdentifier(const DirectLink::FSourceHandle& SourceHandle)
	{
		using namespace DirectLink;

		if (ReceiverEndpoint.IsValid() && SourceHandle.IsValid())
		{
			FRawInfo RawInfo = ReceiverEndpoint->GetRawInfoCopy();

			if (FRawInfo::FDataPointInfo* DataPointInfoPtr = RawInfo.DataPointsInfo.Find(SourceHandle))
			{
				if (FRawInfo::FEndpointInfo* EndPointInfoPtr = RawInfo.EndpointsInfo.Find(DataPointInfoPtr->EndpointAddress))
				{

					return HashCombine(GetTypeHash(SourceHandle), GetTypeHash(DataPointInfoPtr->EndpointAddress));
				}
			}
		}

		return 0xffffffff;
	}

	DirectLink::FSourceHandle FDirectLinkEndpointProxy::GetSourceHandleFromIdentifier(uint32 SourceIdentifier)
	{
		using namespace DirectLink;

		if (ReceiverEndpoint.IsValid())
		{
			FRawInfo RawInfo = ReceiverEndpoint->GetRawInfoCopy();

			for (TPair<FGuid, FRawInfo::FDataPointInfo>& DataPointInfo : RawInfo.DataPointsInfo)
			{
				if (ComputeSourceIdentifier(DataPointInfo.Key) == SourceIdentifier)
				{
					return DataPointInfo.Key;
				}
			}
		}

		return FSourceHandle();
	}

	FString FDirectLinkEndpointProxy::GetEndPointName()
	{
		return ReceiverEndpoint.IsValid() ? EndPointName : TEXT("Invalid");
	}

	const TArray<FDatasmithRuntimeSourceInfo>& FDirectLinkEndpointProxy::GetListOfSources() const
	{
		return LastSources;
	}

	DirectLink::FSourceHandle FDirectLinkEndpointProxy::GetConnection(const DirectLink::FDestinationHandle& DestinationHandle)
	{
		using namespace DirectLink;

		if (ReceiverEndpoint.IsValid())
		{
			FRawInfo RawInfo = ReceiverEndpoint->GetRawInfoCopy();

			for (FRawInfo::FStreamInfo& StreamInfo : RawInfo.StreamsInfo)
			{
				if (/*StreamInfo.bIsActive && */StreamInfo.Destination == DestinationHandle)
				{
					return StreamInfo.Source;
				}
			}
		}

		return FSourceHandle();
	}

	FMD5Hash ComputeRawInfoHash(const DirectLink::FRawInfo& RawInfo)
	{
		using namespace DirectLink;

		TArray<FGuid> Keys;
		RawInfo.DataPointsInfo.GenerateKeyArray(Keys);
		Keys.Sort();

		TArray<uint8> Buffer;
		FMemoryWriter Ar( Buffer );

		for (FGuid& Guid : Keys)
		{
			FRawInfo::FDataPointInfo& DataPointInfo = const_cast<FRawInfo::FDataPointInfo&>(RawInfo.DataPointsInfo[Guid]);
			if (!DataPointInfo.bIsSource)
			{
				continue;
			}

			Ar << Guid;
			Ar << DataPointInfo.Name;
			Ar << DataPointInfo.bIsOnThisEndpoint;

			ensure(RawInfo.EndpointsInfo.Contains(DataPointInfo.EndpointAddress));
			FRawInfo::FEndpointInfo& EndpointInfo = const_cast<FRawInfo::FEndpointInfo&>(RawInfo.EndpointsInfo[DataPointInfo.EndpointAddress]);

			Ar << EndpointInfo.ProcessId;
			Ar << EndpointInfo.ExecutableName;
		}

		FMD5 MD5;
		MD5.Update(Buffer.GetData(), Buffer.Num());

		FMD5Hash Hash;
		Hash.Set(MD5);

		return Hash;
	}

	void FDirectLinkEndpointProxy::OnStateChanged(const DirectLink::FRawInfo& RawInfo)
	{
		using namespace DirectLink;

		// Used to store local destinations with active connections
		TSet<FGuid> FoundActiveConnections;

		FRWScopeLock _(RawInfoCopyLock, SLT_Write);

		for ( const FRawInfo::FStreamInfo& StreamInfo : RawInfo.StreamsInfo)
		{
			if (!(StreamInfo.ConnectionState == DirectLink::EStreamConnectionState::Active 
				|| StreamInfo.ConnectionState == DirectLink::EStreamConnectionState::RequestSent))
			{
				continue;
			}

			if (TSharedPtr<FDestinationProxy>* DestinationProxyPtr = DestinationList.Find(StreamInfo.Destination))
			{
				FoundActiveConnections.Add(StreamInfo.Destination);

				if (StreamInfo.CommunicationStatus.IsTransmitting())
				{
					int32 IsTransmitting = StreamInfo.CommunicationStatus.IsTransmitting() ? 1 : 0;
					float Progress = StreamInfo.CommunicationStatus.GetProgress() * 100.f;
				}
			}
		}

		// Repair any destinations which have lost its connection
		for (TPair<DirectLink::FDestinationHandle, TSharedPtr<FDestinationProxy>>& Entry : DestinationList)
		{
			if (Entry.Value->IsConnected() && !FoundActiveConnections.Contains(Entry.Key))
			{
				// Try to restore connection. Reset if restoration failed
				if(!OpenConnection(Entry.Value->GetConnectedSourceHandle(), Entry.Key))
				{
					Entry.Value->ResetConnection();
				}
			}
		}

		FMD5Hash NewHash = ComputeRawInfoHash(RawInfo);

		if (NewHash == LastHash)
		{
			return;
		}

		// Something has changed with the sources, check what exactly
		LastHash = NewHash;

		// Rebuild list of available public sources
		LastSources.Reset();
		for (const TPair<FGuid, FRawInfo::FDataPointInfo>& MapEntry : RawInfo.DataPointsInfo)
		{
			const FGuid& DataPointId = MapEntry.Key;
			const FRawInfo::FDataPointInfo& DataPointInfo = MapEntry.Value;

			if (DataPointInfo.bIsSource && DataPointInfo.EndpointAddress != RawInfo.ThisEndpointAddress)
			{
				ensure(RawInfo.EndpointsInfo.Contains(DataPointInfo.EndpointAddress));
				const FRawInfo::FEndpointInfo& EndPointInfo = RawInfo.EndpointsInfo[DataPointInfo.EndpointAddress];

				// #ueent_datasmithruntime: Skip remote end points
				if (!EndPointInfo.bIsLocal)
				{
					continue;
				}

				const FString& SourceName = DataPointInfo.Name;

				FString SourceLabel = SourceName + TEXT("-") + EndPointInfo.ExecutableName + TEXT("-") + FString::FromInt((int32)EndPointInfo.ProcessId);
				LastSources.Emplace(SourceLabel, DataPointId);
			}
		}

		bIsDirty = NotifyChange != nullptr;
	}

	void FDirectLinkEndpointProxy::Tick(float DeltaSeconds)
	{
		FRWScopeLock _(RawInfoCopyLock, SLT_Write);
		NotifyChange->Broadcast();
		bIsDirty = false;
	}

	FDestinationProxy::FDestinationProxy(FDatasmithSceneReceiver_ISceneChangeListener* InChangeListener)
		: ChangeListener(InChangeListener)
	{
	}

	FDestinationProxy::~FDestinationProxy()
	{
	}

	bool FDestinationProxy::OpenConnection(uint32 SourceIdentifier)
	{
		return OpenConnection(EndpointProxy->GetSourceHandleFromIdentifier(SourceIdentifier));
	}

	void FDestinationProxy::CloseConnection()
	{
		if (ConnectedSource.IsValid() && Destination.IsValid())
		{
			EndpointProxy->CloseConnection(ConnectedSource, Destination);
			ResetConnection();
		}
	}

	FString FDestinationProxy::GetSourceName()
	{
		return ConnectedSource.IsValid() ? EndpointProxy->GetSourceName(ConnectedSource) : TEXT("None");
	}

	bool FDestinationProxy::RegisterDestination(const TCHAR* StreamName)
	{
		using namespace DirectLink;

		UnregisterDestination();

		EndpointProxy->RegisterDestination(StreamName, this->AsShared() );

		return Destination.IsValid();
	}

	void FDestinationProxy::UnregisterDestination()
	{
		using namespace DirectLink;

		if (Destination.IsValid())
		{
			CloseConnection();

			EndpointProxy->UnregisterDestination(this->AsShared());

			Destination = FDestinationHandle();
		}

		ConnectedSource = FSourceHandle();
	}

	bool FDestinationProxy::OpenConnection(const DirectLink::FSourceHandle& SourceHandle)
	{
		using namespace DirectLink;

		if (SourceHandle.IsValid())
		{
			if (Destination.IsValid())
			{
				if (ConnectedSource.IsValid() && SourceHandle != ConnectedSource)
				{
					EndpointProxy->CloseConnection(ConnectedSource, Destination);
					ConnectedSource = FSourceHandle();
					SceneReceiver.Reset();
				}

				if (ChangeListener)
				{
					SceneReceiver = MakeShared<FDatasmithSceneReceiver>();
					SceneReceiver->SetChangeListener(ChangeListener);
				}

				if (EndpointProxy->OpenConnection(SourceHandle, Destination))
				{
					ConnectedSource = SourceHandle;
				}
			}
		}

		return SourceHandle.IsValid() && SourceHandle == ConnectedSource;
	}
}
