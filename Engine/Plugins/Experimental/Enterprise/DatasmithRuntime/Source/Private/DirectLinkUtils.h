// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "DirectLink/DatasmithSceneReceiver.h"
#include "DirectLinkEndpoint.h"
#include "DirectLinkConnectionRequestHandler.h"

class ADatasmithRuntimeActor;
class UDirectLinkProxy;
struct FDatasmithRuntimeSourceInfo;

namespace DatasmithRuntime
{
	class FDirectLinkEndpointProxy;

	class UDirectLinkProxy* GetDirectLinkProxy();

	/**
	 * Helper class to manage a DirectLink destination and its connection
	 * Used by the ADatasmithRuntimeActor
	 */
	class FDestinationProxy : public DirectLink::IConnectionRequestHandler, public TSharedFromThis<FDestinationProxy>
	{
	public:
		FDestinationProxy(FDatasmithSceneReceiver::ISceneChangeListener* InChangeListener);
		~FDestinationProxy();

		// Begin DirectLink::IConnectionRequestHandler interface
		virtual bool CanOpenNewConnection(const FSourceInformation& /*SourceInfo*/) override
		{
			return true; // Always true
		}

		virtual TSharedPtr<DirectLink::ISceneReceiver> GetSceneReceiver(const FSourceInformation& /*SourceInfo*/) override
		{
			return ChangeListener? SceneReceiver : nullptr;
		}
		// End DirectLink::IConnectionRequestHandler interface

		/** Register as a scene provider to the associated end point */
		bool RegisterDestination(const TCHAR* StreamName);

		/** Unregister as a scene provider to the associated end point */
		void UnregisterDestination();

		bool CanConnect() { return Destination.IsValid(); }

		bool IsConnected() { return Destination.IsValid() && ConnectedSource.IsValid(); }

		/**
		 * Open a connection with the source handle associated to the incoming identifier
		 * @note the identifier is used by the Game UI
		 */
		bool OpenConnection(uint32 SourceIdentifier);

		/** Open a connection with the given source handle */
		bool OpenConnection(const DirectLink::FSourceHandle& SourceHandle);

		/** Close the existing connection */
		void CloseConnection();

		/** Returns the name of the connected source. "None" if there is no connection */
		FString GetSourceName();

		/**
		 * Create the unique end point proxy used by all ADatasmithRuntimeActors
		 * @note: This is called when the DatasmithRuntime module is loaded
		 */
		static void InitializeEndpointProxy();

		/**
		 * Delete the end point proxy
		 * @note: This is called when the DatasmithRuntime module is unloaded
		 */
		static void ShutdownEndpointProxy();

		/** Returns the unique end point proxy. This is used by the Game's UI */
		static FDirectLinkEndpointProxy& GetEndpointProxy()
		{
			return *EndpointProxy;
		}

		static const TArray<FDatasmithRuntimeSourceInfo>& GetListOfSources();

		/** Helper method to reset the members associated with a connection which no longer exists */
		void ResetConnection()
		{
			ConnectedSource = DirectLink::FSourceHandle();
			SceneReceiver.Reset();
		}

		/** helper methods to access members of the proxy */
		TSharedPtr<IDatasmithScene> GetScene()
		{
			return SceneReceiver.IsValid() ? SceneReceiver->GetScene() : TSharedPtr<IDatasmithScene>();
		}

		const DirectLink::FDestinationHandle& GetDestinationHandle() const { return Destination; }

		DirectLink::FDestinationHandle& GetDestinationHandle() { return Destination; }

		const DirectLink::FSourceHandle& GetConnectedSourceHandle() const { return ConnectedSource; }

	private:
		/** Listener to changes happening on the scene associated with DirectLink destination */
		FDatasmithSceneReceiver::ISceneChangeListener* ChangeListener;

		/** Scene receiver wrapping the change listener */
		TSharedPtr<FDatasmithSceneReceiver> SceneReceiver;

		/** Handle to the destination associated with the proxy */
		DirectLink::FDestinationHandle Destination;

		/** Handle to the potential connected source */
		DirectLink::FSourceHandle ConnectedSource;

		/** Pointer to the unique end point proxy */
		static TSharedPtr<FDirectLinkEndpointProxy> EndpointProxy;
	};
}
