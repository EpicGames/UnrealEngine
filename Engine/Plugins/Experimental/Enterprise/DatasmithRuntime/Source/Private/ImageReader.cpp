// Copyright Epic Games, Inc. All Rights Reserved.


#include "DatasmithDefinitions.h"
#include "DatasmithRuntimeUtils.h"

// Borrowed from Engine/Plugins/Enterprise/DatasmithImporter/Source/DatasmithImporter/Private/Utility/DatasmithTextureResize.cpp

#if WITH_FREEIMAGE_LIB
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Math/Float16.h"
#include "Misc/Paths.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif // PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_START
#include "FreeImage.h"
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#   define TCHAR_TO_FICHAR TCHAR_TO_WCHAR
#   define FreeImage_GetFIFFromFilename FreeImage_GetFIFFromFilenameU
#   define FreeImage_GetFileType        FreeImage_GetFileTypeU
#   define FreeImage_Load               FreeImage_LoadU
#   define FreeImage_Save               FreeImage_SaveU
#else
#   define TCHAR_TO_FICHAR TCHAR_TO_UTF8
#endif // PLATFORM_WINDOWS

#endif // WITH_FREEIMAGE_LIB

namespace DatasmithRuntime
{
#if WITH_FREEIMAGE_LIB
	class FFreeImageWrapper
	{
	public:
		static bool IsValid() { return FreeImageDllHandle != nullptr; }

		static void FreeImage_Initialise(); // Loads and inits FreeImage on first call

	private:
		static void* FreeImageDllHandle; // Lazy init on first use, never release for now
	};
#endif

	void ImageReaderInitialize()
	{
#if WITH_FREEIMAGE_LIB
		FFreeImageWrapper::FreeImage_Initialise();
#endif
	}

#if WITH_FREEIMAGE_LIB
	bool GetTextureDataInternal(FIBITMAP* Bitmap, FREE_IMAGE_FORMAT FileType, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData);

	void GetBitmapPixelInfo(FIBITMAP* Bitmap, int32& BitsPerPixel, int32& ChannelCount)
	{
		BitsPerPixel = FreeImage_GetBPP(Bitmap);
		ChannelCount = -1;
		switch (FreeImage_GetImageType(Bitmap))
		{
		case FIT_RGB16:
		case FIT_RGBF:
		{
			ChannelCount = 3;
			break;
		}
		case FIT_RGBAF:
		case FIT_RGBA16:
		{
			ChannelCount = 4;
			break;
		}
		case FIT_UINT16:
		case FIT_INT16:
		{
			ChannelCount = BitsPerPixel / 16;
			break;
		}
		case FIT_UINT32:
		case FIT_INT32:
		case FIT_FLOAT:
		{
			ChannelCount = BitsPerPixel / 32;
			break;
		}
		case FIT_DOUBLE:
		{
			ChannelCount = BitsPerPixel / 64;
			break;
		}
		case FIT_COMPLEX:
		{
			ChannelCount = BitsPerPixel / 128;
			break;
		}
		case FIT_BITMAP:
		{
			if(BitsPerPixel == 16)
			{
				// Make sure we are in RGB16 565 mode or RGB16 555 mode
				// That should be the only cases for a FIF_BITMAP
				unsigned RedMask = FreeImage_GetRedMask(Bitmap);
				unsigned GreenMask = FreeImage_GetGreenMask(Bitmap);
				unsigned BlueMask = FreeImage_GetBlueMask(Bitmap);
				if (  (RedMask == FI16_565_RED_MASK && GreenMask == FI16_565_GREEN_MASK && BlueMask == FI16_565_BLUE_MASK)
					|| (RedMask == FI16_555_RED_MASK && GreenMask == FI16_555_GREEN_MASK && BlueMask == FI16_555_BLUE_MASK))
				{
					ChannelCount = 3;
				}
			}
			else
			{
				ChannelCount = BitsPerPixel == 32 ? 4 : (BitsPerPixel == 24 ? 3 : 1);
			}
			break;
		}
		case FIT_UNKNOWN:
		default:
		{
			break;
		}
		}
	}

	int32 PreviousPowerOfTwo(int32 Reference)
	{
		int32 PreviousValue = 2;
		int32 NextValue = PreviousValue << 1;
		while (Reference > NextValue)
		{
			PreviousValue = NextValue;
			NextValue = PreviousValue << 1;
		}

		return PreviousValue;
	}

	int32 NextPowerOfTwo(int32 Reference)
	{
		return PreviousPowerOfTwo(Reference) << 1;
	}

	int32 NearestPowerOfTwo(int32 Reference)
	{
		int32 PreviousValue = PreviousPowerOfTwo(Reference);

		int32 NextValue = PreviousValue << 1;

		if ((NextValue - Reference)<(Reference - PreviousValue))
		{
			return NextValue;
		}
		else
		{
			return PreviousValue;
		}
	}

	int32 ToPowerOfTwo(int32 Reference, EDSResizeTextureMode ResizeTexturesMode)
	{
		switch (ResizeTexturesMode)
		{
		case EDSResizeTextureMode::NearestPowerOfTwo:
			return NearestPowerOfTwo(Reference);
		case EDSResizeTextureMode::PreviousPowerOfTwo:
			return PreviousPowerOfTwo(Reference);
		case EDSResizeTextureMode::NextPowerOfTwo:
			return NextPowerOfTwo(Reference);
		default:
			return Reference;
		}
	}

	class NinePoints
	{
	public:
		NinePoints(){ aa = ab = ac = ba = bb = bc = ca = cb = cc = 0; }
		float aa, ab, ac, ba, bb, bc, ca, cb, cc;
	};

	int32 Tile(int32 x, int32 MaxTile)
	{
		if (x < 0)
		{
			x = MaxTile - 1;
		}

		if (x >= MaxTile)
		{
			x = 0;
		}

		return x;
	}

	bool PyramidProcess(FIBITMAP* Grey, int32 Level /*= 1*/)
	{
		int32 Height = FreeImage_GetHeight(Grey);
		int32 Width = FreeImage_GetWidth(Grey);

		if (Height <= 8 || Width <= 8 || Level >3)
		{
			return false;
		}

		FIBITMAP* GreyDown = FreeImage_Rescale(Grey, Width / 4, Height / 4, FREE_IMAGE_FILTER::FILTER_LANCZOS3);

		if (!PyramidProcess(GreyDown, Level + 1))
		{
			FreeImage_Unload(GreyDown);
			return true;
		}

		FIBITMAP* GreyUp = FreeImage_Rescale(GreyDown, Width, Height, FREE_IMAGE_FILTER::FILTER_BICUBIC);

		for (int32 Index = 0; Index < Height; ++Index)
		{
			float* Bits = (float*)FreeImage_GetScanLine(Grey, Index);
			float* bitsUp = (float*)FreeImage_GetScanLine(GreyUp, Index);

			for (int32 x = 0; x < Width; x++)
			{
				Bits[x] = (Bits[x] * 0.5f + bitsUp[x] * 0.5f) + bitsUp[x] * 1.5f;
			}
		}

		FreeImage_Unload(GreyDown);
		FreeImage_Unload(GreyUp);

		return true;
	}

	FIBITMAP* ConvertToNormalMap(FIBITMAP* Bump, bool bIsHighRange)
	{
		int32 Height = FreeImage_GetHeight(Bump);
		int32 Width = FreeImage_GetWidth(Bump);
		FIBITMAP* GreyPyramid = FreeImage_ConvertToFloat(Bump);
		FIBITMAP* Normal = FreeImage_ConvertToRGBAF(Bump);
		FreeImage_Unload(Bump);

		if (!PyramidProcess(GreyPyramid, 1))
		{
			return nullptr;
		}

		TArray<NinePoints> Values;

		Values.AddDefaulted(Height * Width);


		float MinVal = 99999;
		float MaxVal = -99999;
		for (int32 Index = 0; Index < Height; ++Index)
		{
			float* Bits = reinterpret_cast< float*>(FreeImage_GetScanLine(GreyPyramid, Index));
			for (int32 x = 0; x < Width; x++)
			{   //memorize colors in a table
				Values[Index*Width + x].bb = Bits[x];
				if (Values[Index*Width + x].bb > MaxVal)
				{
					MaxVal = Values[Index*Width + x].bb;
				}

				if (Values[Index*Width + x].bb < MinVal)
				{
					MinVal = Values[Index*Width + x].bb;
				}
			}
		}

		FreeImage_Unload(GreyPyramid);

		for (int32 Index = 0; Index < Height; ++Index)
		{
			for (int32 x = 0; x < Width; x++)    //equalize
			{
				Values[Index*Width + x].bb = (Values[Index*Width + x].bb - MinVal) / (MaxVal - MinVal);
			}
		}

		for (int32 Index = 0; Index < Height; ++Index)
		{
			for (int32 x = 0; x < Width; x++)
			{   //memorize colors in a table
				Values[Index*Width + x].aa = Values[Tile(Index - 1, Height)*Width + Tile(x - 1, Width)].bb;
				Values[Index*Width + x].ba = Values[Tile(Index - 1, Height)*Width +					 x].bb;
				Values[Index*Width + x].ca = Values[Tile(Index - 1, Height)*Width + Tile(x + 1, Width)].bb;
				Values[Index*Width + x].ab = Values[				  Index*Width + Tile(x - 1, Width)].bb;
				Values[Index*Width + x].cb = Values[				  Index*Width + Tile(x + 1, Width)].bb;
				Values[Index*Width + x].ac = Values[Tile(Index + 1, Height)*Width + Tile(x - 1, Width)].bb;
				Values[Index*Width + x].bc = Values[Tile(Index + 1, Height)*Width +					 x].bb;
				Values[Index*Width + x].cc = Values[Tile(Index + 1, Height)*Width + Tile(x + 1, Width)].bb;
			}
		}

		float Scale = 1.0f;
		float MaxValueN = 0.f;

		for (int32 Index = 0; Index < Height; ++Index)
		{
			FIRGBAF*Bits = (FIRGBAF*)FreeImage_GetScanLine(Normal, Index);
			for (int32 x = 0; x < Width; x++)
			{   //memorize colors in a table
				int32 pos = Index*Width + x;

				Bits[x].red = (Scale* -(Values[pos].cc - Values[pos].ac + 2.0f*(Values[pos].cb - Values[pos].ab) + Values[pos].ca - Values[pos].aa));
				if (FMath::Abs(Bits[x].red) > MaxValueN)
				{
					MaxValueN = FMath::Abs(Bits[x].red);
				}

				Bits[x].green = -(Scale* -(Values[pos].aa - Values[pos].ac + 2.0f*(Values[pos].ba - Values[pos].bc) + Values[pos].ca - Values[pos].cc));
				if (FMath::Abs(Bits[x].green) > MaxValueN)
				{
					MaxValueN = FMath::Abs(Bits[x].green);
				}

				Bits[x].blue = 1.0f;
				Bits[x].alpha = Values[Index*Width + x].bb;
			}
		}

		MaxValueN = 1.0f / MaxValueN;
		for (int32 Index = 0; Index < Height; ++Index)
		{
			FIRGBAF* Bits = (FIRGBAF*)FreeImage_GetScanLine(Normal, Index);

			for (int32 x = 0; x < Width; x++)
			{	//memorize colors in a table
				Bits[x].red = Bits[x].red * MaxValueN;
				if (Bits[x].red > 0)
				{
					Bits[x].red = FMath::Sqrt(Bits[x].red);
				}

				if (Bits[x].red < 0)
				{
					Bits[x].red = -FMath::Sqrt(FMath::Abs(Bits[x].red));
				}

				Bits[x].green = Bits[x].green * MaxValueN;
				if (Bits[x].green > 0)
				{
					Bits[x].green = FMath::Sqrt(Bits[x].green);
				}

				if (Bits[x].green < 0)
				{
					Bits[x].green = -FMath::Sqrt(FMath::Abs(Bits[x].green));
				}

				Bits[x].blue = 0.5f + 0.5f * FMath::Sqrt(1 - (Bits[x].red)*(Bits[x].red) + (Bits[x].green)*(Bits[x].green));

				Bits[x].green = 0.5f + 0.5f * Bits[x].green;
				Bits[x].red = 0.5f + 0.5f * Bits[x].red;
			}
		}

		if (bIsHighRange)
		{
			return Normal;
		}

		FIBITMAP* Normal8 = FreeImage_Allocate(Width, Height, 4 * sizeof(DWORD), 0, 0, 0);

		for (int32 Index = 0; Index < Height; ++Index)
		{
			FIRGBAF* Bits = (FIRGBAF*)FreeImage_GetScanLine(Normal, Index);
			for (int32 x = 0; x < Width; x++)
			{   //memorize colors in a table
				RGBQUAD Color;
				Color.rgbRed = (BYTE)(255.f * (FMath::Max(0.0f, FMath::Min(1.0f, Bits[x].red))));
				Color.rgbGreen = (BYTE)(255.f *(1.0 - (FMath::Max(0.0f, FMath::Min(1.0f, Bits[x].green)))));
				Color.rgbBlue = (BYTE)(255.f * (FMath::Max(0.0f, FMath::Min(1.0f, Bits[x].blue))));
				FreeImage_SetPixelColor(Normal8, x, Index, &Color);
			}
		}
		FreeImage_Unload(Normal);

		return Normal8;
	}

	template<typename T>
	void CopyData(FIBITMAP* Image, FTextureData& TextureData)
	{
		ensure(false);
	}

	template<>
	void CopyData<BYTE>(FIBITMAP* Image, FTextureData& TextureData)
	{
		check(TextureData.ImageData == nullptr);

		const int32 Pitch = FreeImage_GetPitch(Image);
		TextureData.Pitch = Pitch;

		TFunction<void(const BYTE *FIBytes, uint8*)> ProcessPixels;

		const int32 BitsPerPixel = FreeImage_GetBPP(Image);
		if (BitsPerPixel == 32)
		{
			TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);
			TextureData.PixelFormat = EPixelFormat::PF_B8G8R8A8;

#if FREEIMAGE_COLORORDER != FREEIMAGE_COLORORDER_BGR
			ProcessPixels = [Width = TextureData.Width](const BYTE *FIBytes, uint8* Buffer) -> void
			{
				for (int32 Index = 0; Index < Width; ++Index, FIBytes += 4, Buffer += 4)
				{
					Buffer[0] = FIBytes[FI_RGBA_BLUE];
					Buffer[1] = FIBytes[FI_RGBA_GREEN];
					Buffer[2] = FIBytes[FI_RGBA_RED];
					Buffer[3] = FIBytes[FI_RGBA_ALPHA];
				}
			};
#endif // FREEIMAGE_COLORORDER
		}
		else if (BitsPerPixel == 24)
		{
			TextureData.Pitch = TextureData.Width * 4;
			TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);
			TextureData.PixelFormat = EPixelFormat::PF_B8G8R8A8;

			FMemory::Memzero(TextureData.ImageData, TextureData.Height * TextureData.Pitch);

#if FREEIMAGE_COLORORDER == FREEIMAGE_COLORORDER_BGR
			ProcessPixels = [Width = TextureData.Width](const BYTE *FIBytes, uint8* Buffer) -> void
			{

				FI_RGBA_RED;
				for (int32 Index = 0; Index < Width; ++Index, FIBytes += 3, Buffer += 4)
				{
					Buffer[0] = FIBytes[0];
					Buffer[1] = FIBytes[1];
					Buffer[2] = FIBytes[2];
					Buffer[3] = UINT8_MAX;
				}
			};
#else
			ProcessPixels = [Width = TextureData.Width](const BYTE *FIBytes, uint8* Buffer) -> void
			{

				FI_RGBA_RED;
				for (int32 Index = 0; Index < Width; ++Index, FIBytes += 3, Buffer += 4)
				{
					Buffer[0] = FIBytes[FI_RGBA_BLUE];
					Buffer[1] = FIBytes[FI_RGBA_GREEN];
					Buffer[2] = FIBytes[FI_RGBA_RED];
					Buffer[3] = UINT8_MAX;
				}
			};
#endif // FREEIMAGE_COLORORDER
		}
		else if (BitsPerPixel == 16)
		{
			TextureData.Pitch = Pitch;
			TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);
			TextureData.PixelFormat = EPixelFormat::PF_G16;
		}
		else
		{
			ensure(BitsPerPixel == 8);
			TextureData.Pitch = ((TextureData.Width + 3) / 4 ) * 4;
			TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);
			TextureData.PixelFormat = EPixelFormat::PF_G8;
		}

		uint8* Buffer = TextureData.ImageData;

		for(int32 Index = TextureData.Height - 1; Index >= 0 ; --Index, Buffer += TextureData.Pitch)
		{
			const BYTE *FIBytes = (BYTE *)FreeImage_GetScanLine(Image, Index);
			if (ProcessPixels)
			{
				ProcessPixels(FIBytes, Buffer);
			}
			else
			{
				FMemory::Memzero(Buffer, TextureData.Pitch);
				FMemory::Memcpy(Buffer, FIBytes, Pitch);
			}
		}
	}

	template<>
	void CopyData<uint16>(FIBITMAP* Image, FTextureData& TextureData)
	{
		check(TextureData.ImageData == nullptr);

		const int32 Pitch = FreeImage_GetPitch(Image);
		const bool bIsSigned = FreeImage_GetImageType(Image) == FIT_INT16;

		TextureData.PixelFormat = EPixelFormat::PF_G16;
		TextureData.Pitch = Pitch;

		TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * Pitch, 0x20);

		uint8* Buffer = TextureData.ImageData;

		for(int32 Index = TextureData.Height - 1; Index >= 0 ; --Index, Buffer += TextureData.Pitch)
		{
			const BYTE *FIBytes = (BYTE *)FreeImage_GetScanLine(Image, Index);
			if (bIsSigned)
			{
				const int16* FIPixels = (const int16*)FIBytes;
				uint16* Pixels = (uint16*)Buffer;

				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, ++Pixels, ++FIPixels)
				{
					// Clamp at 0 and remap between 0 & UINT16_MAX
					const int32 PixelValue = (FMath::Max((int32)*FIPixels, 0) * 2) - 1;
					*Pixels = (uint16)FMath::Max(PixelValue, 0);
				}
			}
			else
			{
				FMemory::Memcpy(Buffer, FIBytes, Pitch);
			}
		}
	}

	template<>
	void CopyData<uint32>(FIBITMAP* Image, FTextureData& TextureData)
	{
		check(TextureData.ImageData == nullptr);

		const int32 Pitch = FreeImage_GetPitch(Image);
		const bool bIsUnsigned = FreeImage_GetImageType(Image) == FIT_UINT32;

		TextureData.PixelFormat = EPixelFormat::PF_G16;
		TextureData.Pitch = ((TextureData.Width * sizeof(uint16) + 3) / 4) * 4;

		TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);

		uint8* Buffer = TextureData.ImageData;

		for(int32 Index = TextureData.Height - 1; Index >= 0 ; --Index, Buffer += TextureData.Pitch)
		{
			const BYTE *FIBytes = (BYTE *)FreeImage_GetScanLine(Image, Index);
			uint16* Pixels = (uint16*)Buffer;

			if (bIsUnsigned)
			{
				const uint32* FIPixels = (const uint32*)FIBytes;
				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, ++Pixels, ++FIPixels)
				{
					*Pixels = (uint16)(*FIPixels >> 16);
				}
			}
			else
			{
				const int32* FIPixels = (const int32*)FIBytes;
				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, ++Pixels, ++FIPixels)
				{
					// Clamp between 0 and UINT16_MAX
					*Pixels = (uint16)(FMath::Max(*FIPixels, 0) >> 15);
				}
			}
		}
	}

	template<>
	void CopyData<float>(FIBITMAP* Image, FTextureData& TextureData)
	{
		check(TextureData.ImageData == nullptr);

		const bool bIsDouble = FreeImage_GetImageType(Image) == FIT_DOUBLE;

		TextureData.PixelFormat = EPixelFormat::PF_R16F;
		TextureData.Pitch = (((TextureData.Width * sizeof(FFloat16)) + 3) / 4) * 4;

		TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);

		uint8* Buffer = TextureData.ImageData;

		for(int32 Index = TextureData.Height - 1; Index >= 0 ; --Index, Buffer += TextureData.Pitch)
		{
			const BYTE *FIBytes = (BYTE *)FreeImage_GetScanLine(Image, Index);
			FFloat16* Pixels = (FFloat16*)Buffer;

			if (bIsDouble)
			{
				const double* FIPixels = (const double*)FIBytes;
				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, ++Pixels, ++FIPixels)
				{
					*Pixels = (float)(*FIPixels);
				}
			}
			else
			{
				const float* FIPixels = (const float*)FIBytes;
				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, ++Pixels, ++FIPixels)
				{
					*Pixels = *FIPixels;
				}
			}
		}
	}

	template<>
	void CopyData<FIRGB16>(FIBITMAP* Image, FTextureData& TextureData)
	{
		check(TextureData.ImageData == nullptr);

		const int32 Pitch = FreeImage_GetPitch(Image);
		const bool bHasAlpha = FreeImage_GetImageType(Image) == FIT_RGBA16;

		TextureData.PixelFormat = EPixelFormat::PF_R16G16B16A16_UINT;
		TextureData.Pitch = bHasAlpha ? Pitch : TextureData.Width * 4 * sizeof(uint16);

		TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);

		uint8* Buffer = TextureData.ImageData;

		for(int32 Index = TextureData.Height - 1; Index >= 0 ; --Index, Buffer += TextureData.Pitch)
		{
			const BYTE *FIBytes = (BYTE *)FreeImage_GetScanLine(Image, Index);
			if (bHasAlpha)
			{
				FMemory::Memcpy(Buffer, FIBytes, Pitch);
			}
			else
			{
				const FIRGB16* FIPixels = (const FIRGB16*)FIBytes;
				uint16* Pixels = (uint16*)Buffer;
				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, Pixels += 4, ++FIPixels)
				{
					Pixels[0] = (*FIPixels).red;
					Pixels[1] = (*FIPixels).green;
					Pixels[2] = (*FIPixels).blue;
					Pixels[3] = UINT16_MAX;
				}
			}
		}
	}

	template<>
	void CopyData<FIRGBF>(FIBITMAP* Image, FTextureData& TextureData)
	{
		check(TextureData.ImageData == nullptr);

		const int32 Pitch = FreeImage_GetPitch(Image);
		const bool bHasAlpha = FreeImage_GetImageType(Image) == FIT_RGBAF;

		TextureData.PixelFormat = EPixelFormat::PF_A32B32G32R32F;
		TextureData.Pitch = TextureData.Width * 4 * sizeof(float);

		TextureData.ImageData = (uint8*)FMemory::Malloc(TextureData.Height * TextureData.Pitch, 0x20);

		uint8* Buffer = TextureData.ImageData;

		for(int32 Index = TextureData.Height - 1; Index >= 0 ; --Index, Buffer += TextureData.Pitch)
		{
			const BYTE *FIBytes = (BYTE *)FreeImage_GetScanLine(Image, Index);
			float* Pixels = (float*)Buffer;

			if (bHasAlpha)
			{
				const FIRGBAF* FIPixels = (const FIRGBAF*)FIBytes;
				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, Pixels += 4, ++FIPixels)
				{
					Pixels[0] = (*FIPixels).red;
					Pixels[1] = (*FIPixels).green;
					Pixels[2] = (*FIPixels).blue;
					Pixels[3] = (*FIPixels).alpha;
				}
			}
			else
			{
				const FIRGBF* FIPixels = (const FIRGBF*)FIBytes;
				for (int32 PixelIndex = 0; PixelIndex < TextureData.Width; ++PixelIndex, Pixels += 4, ++FIPixels)
				{
					Pixels[0] = (*FIPixels).red;
					Pixels[1] = (*FIPixels).green;
					Pixels[2] = (*FIPixels).blue;
					Pixels[3] = 1.f;
				}
			}
		}
	}

	bool GetTextureDataFromFile(const TCHAR* Filename, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::GetTextureData);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		if (!PlatformFile.FileExists(Filename) || FString(Filename).Len() < 3)
		{
			return false;
		}

		FFreeImageWrapper::FreeImage_Initialise();

		if ( !FFreeImageWrapper::IsValid() )
		{
			//if(!(FFileHelper::LoadFileToArray(OutImageData, Filename) && OutImageData.Num() > 0))
			//{
			//	return EDSTextureUtilsError::FileReadIssue;
			//}

			//return EDSTextureUtilsError::FreeImageNotFound;
			return false;
		}

		//FinalImage format
		FREE_IMAGE_FORMAT FileType = FIF_UNKNOWN;

		//check the file signature and deduce its format
		FileType = FreeImage_GetFileType(TCHAR_TO_FICHAR(Filename), 0);

		//if still unknown, try to guess the file format from the file extension
		if (FileType == FIF_UNKNOWN)
		{
			FileType = FreeImage_GetFIFFromFilename(TCHAR_TO_FICHAR(Filename));
		}

		//if still unknown, return failure
		if (FileType == FIF_UNKNOWN)
		{
			return false;
		}

		//check that the plugin has reading capabilities and load the file
		if (!FreeImage_FIFSupportsReading(FileType))
		{
			return false;
		}

		//pointer to the FinalImage, once loaded
		FIBITMAP* Bitmap = FreeImage_Load(FileType, TCHAR_TO_FICHAR(Filename), 0);

		//if the FinalImage failed to load, return failure
		if (!Bitmap)
		{
			return false;
		}

		return GetTextureDataInternal(Bitmap, FileType, Mode, MaxSize, bGenerateNormalMap, TextureData);
	}

	bool GetTextureDataFromBuffer(TArray<uint8>& Bytes, EDatasmithTextureFormat Format, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::GetTextureData);

		FFreeImageWrapper::FreeImage_Initialise();

		if ( !FFreeImageWrapper::IsValid() )
		{
			//if(!(FFileHelper::LoadFileToArray(OutImageData, Filename) && OutImageData.Num() > 0))
			//{
			//	return EDSTextureUtilsError::FileReadIssue;
			//}

			//return EDSTextureUtilsError::FreeImageNotFound;
			return false;
		}

		// FinalImage format
		FREE_IMAGE_FORMAT FileType = Format == EDatasmithTextureFormat::JPEG ? FIF_JPEG : FIF_PNG;

		// check that the plugin has reading capabilities and load the file
		if (!FreeImage_FIFSupportsReading(FileType))
		{
			return false;
		}

		FIMEMORY *MemoryBuffer = FreeImage_OpenMemory(Bytes.GetData(), Bytes.Num());

		// Ensure the file types match
		ensure(FreeImage_GetFileTypeFromMemory(MemoryBuffer, 0) == FileType);

		// pointer to the FinalImage, once loaded
		FIBITMAP *Bitmap = FreeImage_LoadFromMemory(FileType, MemoryBuffer, 0);

		// Close the memory stream
		FreeImage_CloseMemory(MemoryBuffer);

		// Safe to free array now
		Bytes.Empty();

		return GetTextureDataInternal(Bitmap, FileType, Mode, MaxSize, bGenerateNormalMap, TextureData);
	}

	bool GetTextureDataInternal(FIBITMAP *Bitmap, FREE_IMAGE_FORMAT FileType, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::GetTextureData);

		//if the FinalImage failed to load, return failure
		if (!Bitmap)
		{
			return false;
		}

		//get the FinalImage Width and Height
		const uint32 OriginalWidth = FreeImage_GetWidth(Bitmap);
		const uint32 OriginalHeight = FreeImage_GetHeight(Bitmap);

		{
			//retrieve the FinalImage data
			BYTE* Bits = FreeImage_GetBits(Bitmap);

			//if this somehow one of these failed (they shouldn't), return failure
			if ((Bits == 0) || (OriginalWidth == 0) || (OriginalHeight == 0))
			{
				return false;
			}
		}

		if (bGenerateNormalMap)
		{
			Bitmap = ConvertToNormalMap(Bitmap, FileType == FIF_EXR || FileType == FIF_HDR);
			if (!Bitmap)
			{
				return false;
			}
		}

		uint32 NewWidth = OriginalWidth;
		uint32 NewHeight = OriginalHeight;

		if (OriginalHeight > MaxSize && OriginalHeight > OriginalWidth)
		{
			NewWidth = FMath::RoundToInt((float)MaxSize * (float)OriginalWidth / (float)OriginalHeight);
			NewHeight = MaxSize;
		}
		else if (OriginalWidth > MaxSize && OriginalWidth >= OriginalHeight)
		{
			NewHeight = FMath::RoundToInt((float)MaxSize * (float)OriginalHeight / (float)OriginalWidth);
			NewWidth = MaxSize;
		}

		FIBITMAP* RescaledImage = nullptr;
		if (Mode != EDSResizeTextureMode::NoResize)
		{
			NewWidth = ToPowerOfTwo(NewWidth, Mode);
			NewHeight = ToPowerOfTwo(NewHeight, Mode);

			// Skip resize if not necessary
			RescaledImage = (NewHeight != OriginalHeight || NewWidth != OriginalWidth) ? FreeImage_Rescale(Bitmap, NewWidth, NewHeight, FREE_IMAGE_FILTER::FILTER_LANCZOS3) : Bitmap;
		}
		else
		{
			RescaledImage = Bitmap;
		}

		FIBITMAP* FinalImage = RescaledImage;

		if (bGenerateNormalMap == true)
		{
			FinalImage = FreeImage_ConvertTo32Bits(RescaledImage);
			check(FreeImage_GetImageType(FinalImage) == FIT_BITMAP);
		}

		bool bResult = true;
		TextureData.Width = NewWidth;
		TextureData.Height = NewHeight;
		TextureData.Pitch = FreeImage_GetPitch(FinalImage);
		const int32 BitsPerPixel = FreeImage_GetBPP(FinalImage);
		TextureData.BytesPerPixel = FMath::Max(1, BitsPerPixel / 8);

		check(TextureData.Width && TextureData.Height);

		switch(FreeImage_GetImageType(FinalImage))
		{
			case FIT_BITMAP:
			{
				CopyData<BYTE>(FinalImage, TextureData);
				break;
			}

			case FIT_UINT16:
			case FIT_INT16:
			{
				CopyData<uint16>(FinalImage, TextureData);
				break;
			}

			case FIT_UINT32:
			case FIT_INT32:
			{
				CopyData<uint32>(FinalImage, TextureData);
				break;
			}

			case FIT_FLOAT:
			case FIT_DOUBLE:
			{
				CopyData<float>(FinalImage, TextureData);
				break;
			}

			case FIT_RGB16:
			case FIT_RGBA16:
			{
				CopyData<FIRGB16>(FinalImage, TextureData);
				break;
			}

			case FIT_RGBAF:
			case FIT_RGBF:
			{
				CopyData<FIRGBF>(FinalImage, TextureData);
				break;
			}

			case FIT_COMPLEX:
			default:
			{
				ensure(false);
				bResult = false;
				break;
			}
		}

		if (FinalImage != RescaledImage)
		{
			FreeImage_Unload(FinalImage);
		}

		// Free rescaled FinalImage data if applicable
		if (RescaledImage != Bitmap)
		{
			FreeImage_Unload(RescaledImage);
		}

		//Free FreeImage's copy of the data
		FreeImage_Unload(Bitmap);

		return bResult;
	}

	void* FFreeImageWrapper::FreeImageDllHandle = nullptr;

	void FFreeImageWrapper::FreeImage_Initialise()
	{
		if ( FreeImageDllHandle != nullptr )
		{
			return;
		}

		// Push/PopDllDirectory are soooooo not threadsafe!
		// Must load library in main thread before doing parallel processing
		check(IsInGameThread());

		if ( FreeImageDllHandle == nullptr )
		{
			FString FreeImageDir = FPaths::Combine( FPaths::EngineDir(), TEXT("Binaries/ThirdParty/FreeImage"), FPlatformProcess::GetBinariesSubdirectory() );
			FString FreeImageLibDir = FPaths::Combine( FreeImageDir, TEXT( FREEIMAGE_LIB_FILENAME ));
			FPlatformProcess::PushDllDirectory( *FreeImageDir );
			FreeImageDllHandle = FPlatformProcess::GetDllHandle( *FreeImageLibDir );
			FPlatformProcess::PopDllDirectory( *FreeImageDir );
		}

		if ( FreeImageDllHandle )
		{
			::FreeImage_Initialise();
		}
	}
#else
bool GetTextureDataFromFile(const TCHAR* Filename, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData)
{
	return false;
}
bool GetTextureDataFromBuffer(TArray<uint8>& Bytes, EDatasmithTextureFormat Format, EDSResizeTextureMode Mode, uint32 MaxSize, bool bGenerateNormalMap, FTextureData& TextureData)
{
	return false;
}
#endif
} // End of namespace DatasmithRuntime

#if WITH_FREEIMAGE_LIB
#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif
#endif
