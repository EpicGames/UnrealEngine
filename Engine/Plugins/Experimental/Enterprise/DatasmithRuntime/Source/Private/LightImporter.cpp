// Copyright Epic Games, Inc. All Rights Reserved.


#include "DatasmithRuntimeUtils.h"

#include "DatasmithAreaLightActor.h"
#include "Engine/Scene.h"
#include "IDatasmithSceneElements.h"

#include "Components/DirectionalLightComponent.h"
#include "Components/LightmassPortalComponent.h"
#include "Components/SpotLightComponent.h"
#include "Engine/Blueprint.h"
#include "Engine/DirectionalLight.h"
#include "Engine/PointLight.h"
#include "Engine/SpotLight.h"
#include "Engine/TextureLightProfile.h"

#include "Lightmass/LightmassPortal.h"

namespace DatasmithRuntime
{
	extern const FString TexturePrefix;
	extern const FString MaterialPrefix;
	extern const FString MeshPrefix;

	/** Helper method to set up the properties common to all types of light components */
	void SetupLightComponent(FActorData& ActorData, IDatasmithLightActorElement* LightElement);

	USceneComponent* ImportAreaLightComponent(FActorData& ActorData, IDatasmithAreaLightElement* AreaLightElement, AActor* ParentActor);

	bool FSceneImporter::ProcessLightActorData(FActorData& ActorData, IDatasmithLightActorElement* LightActorElement)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::ProcessLightActorData);

		if (ActorData.HasState(EAssetState::Processed))
		{
			return true;
		}

		if (LightActorElement->GetUseIes() && FCString::Strlen(LightActorElement->GetIesTexturePathName()) > 0)
		{
			if (FSceneGraphId* ElementIdPtr = AssetElementMapping.Find(TexturePrefix + LightActorElement->GetIesTexturePathName()))
			{
				FActionTaskFunction AssignTextureFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
				{
					return this->AssignProfileTexture(Referencer, Cast<UTextureLightProfile>(Object));
				};

				ProcessTextureData(*ElementIdPtr);
				ActorData.AssetId = *ElementIdPtr;

				AddToQueue(EQueueTask::NonAsyncQueue, { AssignTextureFunc, *ElementIdPtr, { EDataType::Actor, ActorData.ElementId, 0 } });
			}
		}

		FActionTaskFunction CreateLightFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
		{
			return this->CreateLightComponent(Referencer.GetId());
		};

		AddToQueue(EQueueTask::NonAsyncQueue, { CreateLightFunc, { EDataType::Actor, ActorData.ElementId, 0 } });
		TasksToComplete |= EWorkerTask::LightComponentCreate;

		ActorData.SetState(EAssetState::Processed);

		return true;
	}

	EActionResult::Type FSceneImporter::AssignProfileTexture(const FReferencer& Referencer, UTextureLightProfile* TextureProfile)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::AssignProfileTexture);

		if (TextureProfile == nullptr)
		{
			ensure(Referencer.Type == (uint8)EDataType::Actor);
			return EActionResult::Failed;
		}

		const FSceneGraphId ActorId = Referencer.GetId();
		ensure(ActorDataList.Contains(ActorId));

		FActorData& ActorData = ActorDataList[ActorId];

		if (!ActorData.HasState(EAssetState::Completed))
		{
			return EActionResult::Retry;
		}

		ActionCounter.Increment();

		if (ULightComponent* LightComponent = ActorData.GetObject<ULightComponent>())
		{
			LightComponent->IESTexture = TextureProfile;
		}
		else if (UChildActorComponent* ChildActorComponent = ActorData.GetObject<UChildActorComponent>())
		{
			if (ADatasmithAreaLightActor* LightShapeActor = Cast< ADatasmithAreaLightActor >(ChildActorComponent->GetChildActor()))
			{
				LightShapeActor->IESTexture = TextureProfile;
			}
		}
		else
		{
			ensure(false);
			return EActionResult::Failed;
		}

		return EActionResult::Succeeded;
	}

	EActionResult::Type FSceneImporter::CreateLightComponent(FSceneGraphId ActorId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::CreateLightComponent);

		FActorData& ActorData = ActorDataList[ActorId];

		IDatasmithLightActorElement* LightElement = static_cast<IDatasmithLightActorElement*>(Elements[ActorId].Get());

		AActor* RootActor = RootComponent->GetOwner();
		check(RootActor);

		USceneComponent* LightComponent = ActorData.GetObject<USceneComponent>();

		if ( LightElement->IsA(EDatasmithElementType::AreaLight) )
		{
			IDatasmithAreaLightElement* AreaLightElement = static_cast<IDatasmithAreaLightElement*>(LightElement);
			LightComponent = ImportAreaLightComponent( ActorData, AreaLightElement, RootActor );
		}
		else if ( LightElement->IsA( EDatasmithElementType::LightmassPortal ) )
		{
			if (LightComponent == nullptr)
			{
				if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None && !LightElement->IsAComponent())
				{
					ALightmassPortal* Actor = CreateActor<ALightmassPortal>(RootActor->GetWorld());
					LightComponent = Actor->GetPortalComponent();
					// Add runtime tag to scene component
					LightComponent->ComponentTags.Add(RuntimeTag);
				}
				else
				{
					LightComponent = CreateComponent< ULightmassPortalComponent >(ActorData, RootActor);
				}
			}
		}
		else if ( LightElement->IsA( EDatasmithElementType::DirectionalLight ) )
		{
			if (LightComponent == nullptr)
			{
				if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None && !LightElement->IsAComponent())
				{
					ADirectionalLight* Actor = CreateActor<ADirectionalLight>(RootActor->GetWorld());
					LightComponent = Actor->GetLightComponent();
					// Add runtime tag to scene component
					LightComponent->ComponentTags.Add(RuntimeTag);
				}
				else
				{
					LightComponent = CreateComponent< UDirectionalLightComponent >(ActorData, RootActor);
				}
			}
		}
		else if ( LightElement->IsA( EDatasmithElementType::SpotLight ) )
		{
			USpotLightComponent* SpotLightComponent = Cast<USpotLightComponent>(LightComponent);

			if (SpotLightComponent == nullptr)
			{
				if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None && !LightElement->IsAComponent())
				{
					ASpotLight* Actor = CreateActor<ASpotLight>(RootActor->GetWorld());
					SpotLightComponent = Cast<USpotLightComponent>(Actor->GetLightComponent());
					// Add runtime tag to scene component
					SpotLightComponent->ComponentTags.Add(RuntimeTag);
				}
				else
				{
					SpotLightComponent = CreateComponent< USpotLightComponent >(ActorData, RootActor);
				}
			}

			if (SpotLightComponent)
			{
				IDatasmithSpotLightElement* SpotLightElement = static_cast<IDatasmithSpotLightElement*>(LightElement);

				SpotLightComponent->InnerConeAngle = SpotLightElement->GetInnerConeAngle();
				SpotLightComponent->OuterConeAngle = SpotLightElement->GetOuterConeAngle();
			}

			LightComponent = SpotLightComponent;
		}
		else if ( LightElement->IsA( EDatasmithElementType::PointLight ) )
		{
			UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(LightComponent);

			if (PointLightComponent == nullptr)
			{
				if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None && !LightElement->IsAComponent())
				{
					APointLight* Actor = CreateActor<APointLight>(RootActor->GetWorld());
					PointLightComponent = Cast<UPointLightComponent>(Actor->GetLightComponent());
					// Add runtime tag to scene component
					PointLightComponent->ComponentTags.Add(RuntimeTag);
				}
				else
				{
					PointLightComponent = CreateComponent< UPointLightComponent >(ActorData, RootActor);
				}
			}

			if (PointLightComponent)
			{
				IDatasmithPointLightElement* PointLightElement = static_cast<IDatasmithPointLightElement*>(LightElement);

				switch ( PointLightElement->GetIntensityUnits() )
				{
				case EDatasmithLightUnits::Candelas:
					PointLightComponent->IntensityUnits = ELightUnits::Candelas;
					break;
				case EDatasmithLightUnits::Lumens:
					PointLightComponent->IntensityUnits = ELightUnits::Lumens;
					break;
				case EDatasmithLightUnits::EV:
					PointLightComponent->IntensityUnits = ELightUnits::EV;
					break;
				default:
					PointLightComponent->IntensityUnits = ELightUnits::Unitless;
					break;
				}

				if ( PointLightElement->GetSourceRadius() > 0.f )
				{
					PointLightComponent->SourceRadius = PointLightElement->GetSourceRadius();
				}

				if ( PointLightElement->GetSourceLength() > 0.f )
				{
					PointLightComponent->SourceLength = PointLightElement->GetSourceLength();
				}

				if ( PointLightElement->GetAttenuationRadius() > 0.f )
				{
					PointLightComponent->AttenuationRadius = PointLightElement->GetAttenuationRadius();
				}
			}

			LightComponent = PointLightComponent;
		}

		ActorData.Object = LightComponent;

		SetupLightComponent( ActorData, LightElement );

		FinalizeComponent(ActorData);

		ActorData.AddState(EAssetState::Completed);

		return LightComponent ? EActionResult::Succeeded : EActionResult::Failed;
	}

	EDatasmithAreaLightActorType GetLightActorTypeForLightType( const EDatasmithAreaLightType LightType )
	{
		EDatasmithAreaLightActorType LightActorType = EDatasmithAreaLightActorType::Point;

		switch ( LightType )
		{
		case EDatasmithAreaLightType::Spot:
			LightActorType = EDatasmithAreaLightActorType::Spot;
			break;

		case EDatasmithAreaLightType::Point:
			LightActorType = EDatasmithAreaLightActorType::Point;
			break;

		case EDatasmithAreaLightType::IES_DEPRECATED:
			LightActorType = EDatasmithAreaLightActorType::Point;
			break;

		case EDatasmithAreaLightType::Rect:
			LightActorType = EDatasmithAreaLightActorType::Rect;
			break;
		}

		return LightActorType;
	}

	USceneComponent* ImportAreaLightComponent( FActorData& ActorData, IDatasmithAreaLightElement* AreaLightElement, AActor* ParentActor )
	{
		FSoftObjectPath LightShapeBlueprintRef = FSoftObjectPath( TEXT("/DatasmithContent/Datasmith/DatasmithArealight.DatasmithArealight") );
		UBlueprint* LightShapeBlueprint = Cast< UBlueprint >( LightShapeBlueprintRef.TryLoad() );

		if ( LightShapeBlueprint )
		{
			UChildActorComponent* ChildActorComponent = ActorData.GetObject<UChildActorComponent>();

			if (ChildActorComponent == nullptr)
			{
				ChildActorComponent = CreateComponent< UChildActorComponent >(ActorData, ParentActor);

				ChildActorComponent->SetChildActorClass( TSubclassOf< AActor > ( LightShapeBlueprint->GeneratedClass ) );
			}

			ChildActorComponent->DestroyChildActor();

			auto LightShapeCustomizer = [AreaLightElement](AActor* Actor)
			{
				if (ADatasmithAreaLightActor* LightShapeActor = Cast<ADatasmithAreaLightActor>(Actor))
				{
					LightShapeActor->LightType = GetLightActorTypeForLightType( AreaLightElement->GetLightType() );
					LightShapeActor->LightShape = (EDatasmithAreaLightActorShape)AreaLightElement->GetLightShape();
					LightShapeActor->Dimensions = FVector2D( AreaLightElement->GetLength(), AreaLightElement->GetWidth() );
					LightShapeActor->Color = AreaLightElement->GetColor();
					LightShapeActor->Intensity = AreaLightElement->GetIntensity();
					LightShapeActor->IntensityUnits = (ELightUnits)AreaLightElement->GetIntensityUnits();

					if ( AreaLightElement->GetUseTemperature() )
					{
						LightShapeActor->Temperature = AreaLightElement->GetTemperature();
					}

					if ( AreaLightElement->GetUseIes() )
					{
						LightShapeActor->bUseIESBrightness = AreaLightElement->GetUseIesBrightness();
						LightShapeActor->IESBrightnessScale = AreaLightElement->GetIesBrightnessScale();
						LightShapeActor->Rotation = AreaLightElement->GetIesRotation().Rotator();
					}

					if ( AreaLightElement->GetSourceRadius() > 0.f )
					{
						LightShapeActor->SourceRadius = AreaLightElement->GetSourceRadius();
					}

					if ( AreaLightElement->GetSourceLength() > 0.f )
					{
						LightShapeActor->SourceLength = AreaLightElement->GetSourceLength();
					}

					if ( AreaLightElement->GetAttenuationRadius() > 0.f )
					{
						LightShapeActor->AttenuationRadius = AreaLightElement->GetAttenuationRadius();
					}
				}
			};

			ChildActorComponent->CreateChildActor(MoveTemp(LightShapeCustomizer));

			if (ADatasmithAreaLightActor* LightShapeActor = Cast<ADatasmithAreaLightActor>(ChildActorComponent->GetChildActor()))
			{
				RenameObject(LightShapeActor, AreaLightElement->GetName());
#if WITH_EDITOR
				LightShapeActor->SetActorLabel(AreaLightElement->GetLabel());
#endif
				return ChildActorComponent;
			}
		}

		return nullptr;
	}

	void SetupLightComponent( FActorData& ActorData, IDatasmithLightActorElement* LightElement )
	{
		if ( ULightComponent* LightComponent = ActorData.GetObject<ULightComponent>() )
		{
			// Light component is using its visibility property to indicate if it is active or not
			LightElement->SetVisibility(LightElement->IsEnabled());

			LightComponent->Intensity = LightElement->GetIntensity();
			LightComponent->CastShadows = true;
			LightComponent->LightColor = LightElement->GetColor().ToFColor( true );
			LightComponent->bUseTemperature = LightElement->GetUseTemperature();
			LightComponent->Temperature = LightElement->GetTemperature();

			// #ue_datasmithruntime: material function not supported yet
			//if ( LightElement->GetLightFunctionMaterial().IsValid() )
			//{
			//	FString BaseName = LightElement->GetLightFunctionMaterial()->GetName();
			//	FString MaterialName = FPaths::Combine( MaterialsFolderPath, BaseName + TEXT(".") + BaseName );
			//	UMaterialInterface* Material = Cast< UMaterialInterface >( FSoftObjectPath( *MaterialName ).TryLoad() );

			//	if ( Material )
			//	{
			//		LightComponent->LightFunctionMaterial = Material;
			//	}
			//}

			if (UPointLightComponent* PointLightComponent = Cast<UPointLightComponent>(LightComponent))
			{
				if ( LightElement->GetUseIes() ) // For IES lights that are not area lights, the IES rotation should be baked into the light transform
				{
					PointLightComponent->bUseIESBrightness = LightElement->GetUseIesBrightness();
					PointLightComponent->IESBrightnessScale = LightElement->GetIesBrightnessScale();

					const FQuat Rotation = LightElement->GetRotation() * LightElement->GetIesRotation();

					ActorData.WorldTransform = FTransform( Rotation, LightElement->GetTranslation(), LightElement->GetScale() );
				}
			}

			LightComponent->UpdateColorAndBrightness();
		}
	}

} // End of namespace DatasmithRuntime
