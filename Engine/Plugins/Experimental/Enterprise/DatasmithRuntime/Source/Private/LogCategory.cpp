// Copyright Epic Games, Inc. All Rights Reserved.

#include "LogCategory.h"
#include "Logging/LogCategory.h"

DEFINE_LOG_CATEGORY(LogDatasmithRuntime)
