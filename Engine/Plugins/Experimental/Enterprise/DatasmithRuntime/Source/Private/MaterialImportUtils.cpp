// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialImportUtils.h"


#include "DatasmithMaterialElements.h"
#include "Engine/Texture.h"
#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "SceneImporter.h"

namespace DatasmithRuntime
{
	const TCHAR* MATERIAL_HOST = TEXT("_Runtime_");

	namespace EPbrTexturePropertySlot
	{
		enum Type
		{
			BumpMapSlot        = 0,
			DiffuseMapSlot     = 1,
			NormalMapSlot      = 2,
			EmissiveMapSlot    = 3,
			RoughnessMapSlot   = 4,
			OpacityMapSlot     = 5,
			RefractionMapSlot  = 6,
			MetallicMapSlot    = 7,
			SpecularMapSlot    = 8,
			MaxSlots           = 9,
		};
	}

	FName PbrTexturePropertyNames[EPbrTexturePropertySlot::MaxSlots] =
	{
		TEXT("BumpMap"),
		TEXT("DiffuseMap"),
		TEXT("NormalMap"),
		TEXT("EmissiveMap"),
		TEXT("RoughnessMap"),
		TEXT("OpacityMap"),
		TEXT("RefractionMap"),
		TEXT("MetallicMap"),
		TEXT("SpecularMap"),
	};

	constexpr const TCHAR* OpaqueMaterialPath = TEXT("/DatasmithRuntime/Materials/M_PbrOpaque.M_PbrOpaque");
	constexpr const TCHAR* OpaqueMaterialPath_2Sided = TEXT("/DatasmithRuntime/Materials/M_PbrOpaque_2Sided.M_PbrOpaque_2Sided");
	constexpr const TCHAR* TranslucentMaterialPath = TEXT("/DatasmithRuntime/Materials/M_PbrTranslucent.M_PbrTranslucent");
	constexpr const TCHAR* TranslucentMaterialPath_2Sided = TEXT("/DatasmithRuntime/Materials/M_PbrTranslucent_2Sided.M_PbrTranslucent_2Sided");

	struct FMaterialParameters
	{
		TMap< FName, int32 > VectorParams;
		TMap< FName, int32 > ScalarParams;
		TMap< FName, int32 > TextureParams;
		TMap< FName, int32 > BoolParams;
	};

	extern const FString TexturePrefix;
	extern const FString MaterialPrefix;
	extern const FString MeshPrefix;

	static TMap< UMaterialInterface*, FMaterialParameters > MaterialParametersCache;

	const FMaterialParameters& GetMaterialParameters(UMaterialInterface* Material)
	{
		check(Material);

		if (const FMaterialParameters* Parameters = MaterialParametersCache.Find(Material))
		{
			return *Parameters;
		}

		FMaterialParameters Parameters;

		TArray<FMaterialParameterInfo> ParameterInfos;
		TArray<FGuid> ParameterIds;
		Material->GetAllScalarParameterInfo(ParameterInfos, ParameterIds);

		for (FMaterialParameterInfo& ParameterInfo : ParameterInfos)
		{
			Parameters.ScalarParams.Add(ParameterInfo.Name, ParameterInfo.Index);
		}

		ParameterInfos.Reset();
		ParameterIds.Reset();
		Material->GetAllVectorParameterInfo(ParameterInfos, ParameterIds);

		for (FMaterialParameterInfo& ParameterInfo : ParameterInfos)
		{
			Parameters.VectorParams.Add(ParameterInfo.Name, ParameterInfo.Index);
		}

		ParameterInfos.Reset();
		ParameterIds.Reset();
		Material->GetAllTextureParameterInfo(ParameterInfos, ParameterIds);

		for (FMaterialParameterInfo& ParameterInfo : ParameterInfos)
		{
			Parameters.TextureParams.Add(ParameterInfo.Name, ParameterInfo.Index);
		}

#if WITH_EDITORONLY_DATA
		ParameterInfos.Reset();
		ParameterIds.Reset();
		Material->GetAllStaticSwitchParameterInfo(ParameterInfos, ParameterIds);

		for (FMaterialParameterInfo& ParameterInfo : ParameterInfos)
		{
			Parameters.BoolParams.Add(ParameterInfo.Name, ParameterInfo.Index);
		}
#endif

		FMaterialParameters& ParametersRef = MaterialParametersCache.Add(Material);
		ParametersRef = MoveTemp(Parameters);

		return ParametersRef;
	}

	int32 ProcessMaterialElement(TSharedPtr< IDatasmithMaterialInstanceElement > ReferenceMaterialElement, FTextureCallback TextureCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::ProcessMaterialElement);

		// Must be updated if FDatasmithMaterialImporter::GetMaterialRequirements changes
		int32 MaterialRequirement = EMaterialRequirements::RequiresNormals | EMaterialRequirements::RequiresTangents;

		if (!ReferenceMaterialElement.IsValid())
		{
			return MaterialRequirement;
		}

		TSharedPtr< FDatasmithReferenceMaterialSelector > MaterialSelector = FDatasmithReferenceMaterialManager::Get().GetSelector(MATERIAL_HOST);

		UMaterialInterface* Material = nullptr;

		if (ReferenceMaterialElement->GetMaterialType() == EDatasmithReferenceMaterialType::Custom)
		{
			FDatasmithReferenceMaterial CustomReferenceMaterial; // ReferenceMaterial might point on this so keep them in the same scope

			CustomReferenceMaterial.FromSoftObjectPath(FSoftObjectPath(ReferenceMaterialElement->GetCustomMaterialPathName()));

			if (CustomReferenceMaterial.IsValid())
			{
				Material = CustomReferenceMaterial.GetMaterial();
			}
		}
		else if (MaterialSelector.IsValid() && MaterialSelector->IsValid())
		{
			const FDatasmithReferenceMaterial& ReferenceMaterial = MaterialSelector->GetReferenceMaterial(ReferenceMaterialElement);

			if (ReferenceMaterial.IsValid())
			{
				Material = ReferenceMaterial.GetMaterial();
			}
		}

		if (Material)
		{
			const TMap< FName, int32 >& TextureParams = GetMaterialParameters(Material).TextureParams;

			for (int Index = 0; Index < ReferenceMaterialElement->GetPropertiesCount(); ++Index)
			{
				const TSharedPtr< IDatasmithKeyValueProperty > Property = ReferenceMaterialElement->GetProperty(Index);
				const FName PropertyName(Property->GetName());

				if (TextureParams.Contains(PropertyName))
				{
					FString TextureName;
					if ( MaterialSelector->GetTexture( Property, TextureName ) )
					{
						TextureCallback(TexturePrefix + TextureName, Index);
					}
				}
			}
		}

		return MaterialRequirement;
	}

	bool LoadReferenceMaterial(UMaterialInstanceDynamic* MaterialInstance, TSharedPtr<IDatasmithMaterialInstanceElement>& MaterialElement)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::LoadReferenceMaterial);

		FDatasmithReferenceMaterialManager& MaterialManager = FDatasmithReferenceMaterialManager::Get();
		const FString Host = MaterialManager.GetHostFromString( MATERIAL_HOST );
		TSharedPtr< FDatasmithReferenceMaterialSelector > MaterialSelector = MaterialManager.GetSelector( MATERIAL_HOST );

		UMaterialInterface* ParentMaterial = nullptr;

		{
			if ( MaterialElement->GetMaterialType() == EDatasmithReferenceMaterialType::Custom )
			{
				FDatasmithReferenceMaterial CustomReferenceMaterial;

				CustomReferenceMaterial.FromSoftObjectPath( FSoftObjectPath( MaterialElement->GetCustomMaterialPathName() ) );
				ParentMaterial = CustomReferenceMaterial.GetMaterial();
			}
			else if ( MaterialSelector.IsValid() )
			{
				const FDatasmithReferenceMaterial& DatasmithReferenceMaterial = MaterialSelector->GetReferenceMaterial(MaterialElement);
				ParentMaterial = DatasmithReferenceMaterial.GetMaterial();
			}
		}

		if (ParentMaterial == nullptr)
		{
			return false;
		}

		MaterialInstance->Parent = ParentMaterial;

		const FMaterialParameters& MaterialParameters = GetMaterialParameters(ParentMaterial);

		for (int Index = 0; Index < MaterialElement->GetPropertiesCount(); ++Index)
		{
			const TSharedPtr< IDatasmithKeyValueProperty >& Property = MaterialElement->GetProperty(Index);
			FName PropertyName(Property->GetName());

			// Vector Params
			if ( MaterialParameters.VectorParams.Contains(PropertyName) )
			{
				FLinearColor Color;
				if ( MaterialSelector->GetColor( Property, Color ) )
				{
					MaterialInstance->SetVectorParameterValue(PropertyName, Color);
				}
			}
			// Scalar Params
			else if ( MaterialParameters.ScalarParams.Contains(PropertyName) )
			{
				float Value;
				if ( MaterialSelector->GetFloat( Property, Value ) )
				{
					MaterialInstance->SetScalarParameterValue(PropertyName, Value);
				}
			}
		}

		return true;
	}

	TSharedPtr<IDatasmithUEPbrMaterialElement> ValidatePbrMaterial( TSharedPtr<IDatasmithUEPbrMaterialElement> PbrMaterialElement, FSceneImporter& SceneImporter )
	{
		// Assuming Pbr materials using material attributes are layered materials
		if (PbrMaterialElement->GetUseMaterialAttributes())
		{
			for (int32 Index = 0; Index < PbrMaterialElement->GetExpressionsCount(); ++Index)
			{
				IDatasmithMaterialExpression* Expression = PbrMaterialElement->GetExpression(Index);
				if (Expression && Expression->IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
				{
					IDatasmithMaterialExpressionFunctionCall* FunctionCall = static_cast<IDatasmithMaterialExpressionFunctionCall*>(Expression);
					TSharedPtr< IDatasmithElement > ElementPtr = SceneImporter.GetElementFromName(MaterialPrefix + FunctionCall->GetFunctionPathName());
					ensure(ElementPtr.IsValid() && ElementPtr->IsA( EDatasmithElementType::UEPbrMaterial ));

					return StaticCastSharedPtr<IDatasmithUEPbrMaterialElement>(ElementPtr);
				}
			}
		}

		return PbrMaterialElement;
	}

	// Class that try to evaluate Datasmith PBR materials so we cant create a Twinmotion material that fit relatively well.
	// Twinmotion cannot implement real PBR shader. Because we cannot compile them.
	class FDatasmithInputValue // TODO: #todo_tm [CodingStyle] RichardY CDatasmithPBRMaterialEvaluator
	{
	public:
		// Texture informations evaluated
		class FTexture
		{
		public:
			FString         TextureName;
			int32           CoordinateIndex = 0;
			bool            bMirrorU = false;
			bool            bMirrorV = false;
			float           Fading = 1.0f;
			float           UTiling = 1.0f;
			float           VTiling = 1.0f;
			float           UTilingPivot = 0.5f;
			float           VTilingPivot = 0.5f;
			float           UOffset = 0.0f;
			float           VOffset = 0.0f;
			float           URotationPivot = 0.5f;
			float           VRotationPivot = 0.5f;
			float           Rotation = 0.0f;
			float           UVOffset = 0.0f;
			bool            bUseAlpha = false;
			bool			bIsBumpMap = false;

			// Constructor
			FTexture(const TCHAR* InName): TextureName(InName) {}
		};

		FLinearColor    Numeric = FLinearColor(ForceInit);    // Numeric value (scalar, color or vector)
		bool            bNumericValid = false;  // If numerical value is valid
		TUniquePtr<FTexture> Texture;     // If we have a Texture defined TODO: #todo_tm [CodingStyle] RichardY TUniquePtr

												// Empty constructor
		FDatasmithInputValue() {}

		// Constructor with an expression
		FDatasmithInputValue(const IDatasmithExpressionInput& InInput)
		{
			EvaluateInput(InInput);
		}

		// Evaluate this expression
		void EvaluateInput(const IDatasmithExpressionInput& InInput); // TODO: #todo_tm [CodingStyle] RichardY EvaluateExpression

																	  // Set scalar value (value is set for all components)
		void Set(float InScalar) { Numeric.R = Numeric.G = Numeric.B = Numeric.A = InScalar; bNumericValid = true; }

		// Set a numeric value (color or vector)
		void Set(const FLinearColor& InNumeric) { Numeric = InNumeric; bNumericValid = true; }

		// Add Texture information for the specified Texture name
		void Set(const TCHAR* InTextureName) { Texture = MakeUnique<FTexture>(InTextureName); }

		// Get scalar value (We use index 0, but normally all components are equals)
		float GetScalar() const { return Numeric.Component(0); }
	};

	int32 ProcessMaterialElement( IDatasmithUEPbrMaterialElement* PbrMaterialElement, FTextureCallback TextureCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::ProcessPbrMaterialElement);

		// Must be updated if FDatasmithMaterialImporter::GetMaterialRequirements changes
		int32 MaterialRequirement = EMaterialRequirements::RequiresNormals | EMaterialRequirements::RequiresTangents;

		if (!PbrMaterialElement)
		{
			return MaterialRequirement;
		}

		TFunction<void(const IDatasmithExpressionInput&)> ParseExpression;
		ParseExpression = [&TextureCallback, &ParseExpression](const IDatasmithExpressionInput& Input) -> void
		{
			if (const IDatasmithMaterialExpression* MaterialExpression = Input.GetExpression())
			{
				if (MaterialExpression->IsSubType(EDatasmithMaterialExpressionType::Texture))
				{
					const IDatasmithMaterialExpressionTexture* TextureExpression = static_cast<const IDatasmithMaterialExpressionTexture*>(MaterialExpression);
					TextureCallback(TexturePrefix + TextureExpression->GetTexturePathName(), -1);
				}

				for (int32 InputIndex = 0; InputIndex < MaterialExpression->GetInputCount(); ++InputIndex)
				{
					if (MaterialExpression->GetInput(InputIndex))
					{
						ParseExpression(*MaterialExpression->GetInput(InputIndex));
					}
				}
			}
		};

		TFunction<void(IDatasmithExpressionInput&, int32)> FindTexture;
		FindTexture = [TextureCallback](IDatasmithExpressionInput& Input, int32 SlotIndex) -> void
		{
			FDatasmithInputValue InputValue(Input);

			if (InputValue.Texture && !InputValue.Texture->TextureName.IsEmpty())
			{
				if (!InputValue.Texture->TextureName.StartsWith(TEXT("/")))
				{
					TextureCallback(TexturePrefix + InputValue.Texture->TextureName, InputValue.Texture->bIsBumpMap ? EPbrTexturePropertySlot::BumpMapSlot : SlotIndex);
				}
			}
		};

		FindTexture(PbrMaterialElement->GetBaseColor(), EPbrTexturePropertySlot::DiffuseMapSlot);
		FindTexture(PbrMaterialElement->GetOpacity(), EPbrTexturePropertySlot::OpacityMapSlot);
		FindTexture(PbrMaterialElement->GetNormal(), EPbrTexturePropertySlot::NormalMapSlot);
		FindTexture(PbrMaterialElement->GetRoughness(), EPbrTexturePropertySlot::RoughnessMapSlot);
		FindTexture(PbrMaterialElement->GetRefraction(), EPbrTexturePropertySlot::RefractionMapSlot);
		FindTexture(PbrMaterialElement->GetEmissiveColor(), EPbrTexturePropertySlot::EmissiveMapSlot);
		FindTexture(PbrMaterialElement->GetMetallic(), EPbrTexturePropertySlot::MetallicMapSlot);
		FindTexture(PbrMaterialElement->GetSpecular(), EPbrTexturePropertySlot::SpecularMapSlot);

		return MaterialRequirement;
	}

	// Borrowed from TM, CDatasmithMaterialImportEntry::EvalUEPbrMaterial()
	bool LoadPbrMaterial(IDatasmithUEPbrMaterialElement& UEPbrMaterial, UMaterialInstanceDynamic* MaterialInstance)
	{
		if (!UEPbrMaterial.GetMaterialFunctionOnly())
		{
			TFunction<void(const FDatasmithInputValue::FTexture*, int32)> SetTextureParams;
			SetTextureParams = [MaterialInstance](const FDatasmithInputValue::FTexture* Texture, int32 SlotIndex) -> void
			{
				if (Texture && !Texture->TextureName.IsEmpty())
				{
					if (Texture->TextureName.StartsWith(TEXT("/")))
					{
						FSoftObjectPath SoftObject(*Texture->TextureName);
						if (UTexture* TextureAsset = Cast<UTexture>(SoftObject.TryLoad()))
						{
							MaterialInstance->SetTextureParameterValue(PbrTexturePropertyNames[SlotIndex], TextureAsset);
						}
					}

					FString RootName(PbrTexturePropertyNames[SlotIndex].ToString());
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("Fading")), Texture->Fading);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_UOffset")), Texture->UOffset);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_VOffset")), Texture->VOffset);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_UTiling")), Texture->UTiling);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_VTiling")), Texture->VTiling);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_UTilingPivot")), Texture->UTilingPivot);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_VTilingPivot")), Texture->VTilingPivot);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_RotAngle")), Texture->Rotation);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_URotPivot")), Texture->URotationPivot);
					MaterialInstance->SetScalarParameterValue(*(RootName + TEXT("_VRotPivot")), Texture->VRotationPivot);
				}
			};

			FDatasmithInputValue OpacityValue(UEPbrMaterial.GetOpacity());
			FDatasmithInputValue RefractionValue(UEPbrMaterial.GetRefraction());

			bool bNeedsTranslucent = (OpacityValue.bNumericValid && OpacityValue.GetScalar() != 1.0f) || OpacityValue.Texture || RefractionValue.bNumericValid || RefractionValue.Texture;

			if (UEPbrMaterial.GetTwoSided())
			{
				FSoftObjectPath SoftObject(bNeedsTranslucent ? TranslucentMaterialPath_2Sided : OpaqueMaterialPath_2Sided);
				MaterialInstance->Parent = Cast<UMaterial>(SoftObject.TryLoad());
			}
			else
			{
				FSoftObjectPath SoftObject(bNeedsTranslucent ? TranslucentMaterialPath : OpaqueMaterialPath);
				MaterialInstance->Parent = Cast<UMaterial>(SoftObject.TryLoad());
			}
			check(MaterialInstance->Parent);

			// PBR Material are too complex to be fully supported with simple Twinmotion material
			{
				if (OpacityValue.bNumericValid && OpacityValue.GetScalar() != 1.0f)
				{
					MaterialInstance->SetScalarParameterValue(TEXT("Opacity"), OpacityValue.GetScalar());
				}

				SetTextureParams(OpacityValue.Texture.Get(), EPbrTexturePropertySlot::OpacityMapSlot);
			}

			{
				if (RefractionValue.bNumericValid && RefractionValue.GetScalar() != 0.0f)
				{
					MaterialInstance->SetScalarParameterValue(TEXT("Refraction"), RefractionValue.GetScalar());
				}

				SetTextureParams(RefractionValue.Texture.Get(), EPbrTexturePropertySlot::RefractionMapSlot);
			}

			{
				FDatasmithInputValue InputValue(UEPbrMaterial.GetBaseColor());

				if (InputValue.bNumericValid)
				{
					MaterialInstance->SetVectorParameterValue(TEXT("DiffuseColor"), InputValue.Numeric);
				}

				SetTextureParams(InputValue.Texture.Get(), EPbrTexturePropertySlot::DiffuseMapSlot);
			}

			{
				FDatasmithInputValue InputValue(UEPbrMaterial.GetRoughness());

				if (InputValue.bNumericValid)
				{
					MaterialInstance->SetScalarParameterValue(TEXT("Roughness"), InputValue.GetScalar());
				}

				SetTextureParams(InputValue.Texture.Get(), EPbrTexturePropertySlot::RoughnessMapSlot);
			}

			{
				FDatasmithInputValue InputValue(UEPbrMaterial.GetNormal());

				if (InputValue.Texture)
				{
					if (InputValue.Texture->bIsBumpMap)
					{
						SetTextureParams(InputValue.Texture.Get(), EPbrTexturePropertySlot::BumpMapSlot);
					}
					else
					{
						SetTextureParams(InputValue.Texture.Get(), EPbrTexturePropertySlot::NormalMapSlot);
					}
				}
			}

			{
				FDatasmithInputValue InputValue(UEPbrMaterial.GetMetallic());

				if (InputValue.bNumericValid && InputValue.GetScalar() != 0.0f)
				{
					MaterialInstance->SetScalarParameterValue(TEXT("Metallic"), InputValue.GetScalar());
				}

				SetTextureParams(InputValue.Texture.Get(), EPbrTexturePropertySlot::MetallicMapSlot);
			}

			{
				FDatasmithInputValue InputValue(UEPbrMaterial.GetSpecular());

				if (InputValue.bNumericValid && InputValue.GetScalar() != 0.0f)
				{
					MaterialInstance->SetScalarParameterValue(TEXT("Specular"), InputValue.GetScalar());
				}

				SetTextureParams(InputValue.Texture.Get(), EPbrTexturePropertySlot::SpecularMapSlot);
			}

			// GetWorldDisplacement() & GetAmbientOcclusion() Not supported

			return true;
		}

		return false;
	}

	// Code borrowed from TwinMotion: Project\Source\TwinmotionCore\Private\Import\DatasmithInputValue.cpp
	// Static class that traverse PBR materials expressions trying to evaluated it
	class FDatasmithExpressionEvaluator
	{
	public:
		// Evaluate this expression input
		static void EvaluateInput(const IDatasmithExpressionInput& InExpressionInput, FDatasmithInputValue& OutValue);

	protected:
		// Evaluate this expression input (input can be nullptr)
		static void EvaluateInput(const IDatasmithExpressionInput* InExpressionInput, FDatasmithInputValue& OutValue);

		// Evaluate this expression
		static void EvaluateExpression(const IDatasmithMaterialExpression& InExpression, FDatasmithInputValue& OutValue);

		// Evaluate is a basic Function (multiply, add, ...)
		static void EvalExpressionGeneric(const IDatasmithMaterialExpressionGeneric& InGenericExpression, FDatasmithInputValue& OutValue);

		// Evaluate is a Unreal Engine Function
		static void EvalExpressionFctCall(const IDatasmithMaterialExpressionFunctionCall& InFctCallExpression, FDatasmithInputValue& OutValue);

		// Evaluate all children of this expression
		static void EvalExpressionChildren(const IDatasmithMaterialExpression& InExpression, FDatasmithInputValue* InOutInputs);
	};


	// Evaluate this expression input (input can be nullptr)
	void FDatasmithExpressionEvaluator::EvaluateInput(const IDatasmithExpressionInput* InExpressionInput, FDatasmithInputValue& OutValue)
	{
		// Do we have an input ?
		if (InExpressionInput == nullptr)
		{
			return;
		}
		EvaluateInput(*InExpressionInput, OutValue);
	}


	// Evaluate this expression input
	void FDatasmithExpressionEvaluator::EvaluateInput(const IDatasmithExpressionInput& InExpressionInput, FDatasmithInputValue& OutValue)
	{
		// Do we have an expression ?
		const IDatasmithMaterialExpression* ExpressionPtr = InExpressionInput.GetExpression();
		if (ExpressionPtr == nullptr)
		{
			return;
		}

		// Evaluate the expression
		EvaluateExpression(*ExpressionPtr, OutValue);

		// Select component
		int32 outputIndex = InExpressionInput.GetOutputIndex();
		if (outputIndex != 0)
		{
			OutValue.Numeric.R = OutValue.Numeric.G = OutValue.Numeric.B = OutValue.Numeric.A = OutValue.Numeric.Component(outputIndex - 1);
			if (outputIndex == 4 && OutValue.Texture)
			{
				OutValue.Texture->bUseAlpha = true;
			}
		}
	}


	// Evaluate this expression
	void FDatasmithExpressionEvaluator::EvaluateExpression(const IDatasmithMaterialExpression& InExpression, FDatasmithInputValue& OutValue)
	{
		// Call the evaluator depending of the expression type
		if (InExpression.IsSubType(EDatasmithMaterialExpressionType::ConstantBool))
		{
			const IDatasmithMaterialExpressionBool& boolExpression = static_cast<const IDatasmithMaterialExpressionBool&>(InExpression);
			OutValue.Set(boolExpression.GetBool() ? 1.0f : 0.0f);
		}
		else if (InExpression.IsSubType(EDatasmithMaterialExpressionType::ConstantColor))
		{
			const IDatasmithMaterialExpressionColor& ColorExpression = static_cast<const IDatasmithMaterialExpressionColor&>(InExpression);
			OutValue.Set(ColorExpression.GetColor());
		}
		else if (InExpression.IsSubType(EDatasmithMaterialExpressionType::ConstantScalar))
		{
			const IDatasmithMaterialExpressionScalar& ScalarExpression = static_cast<const IDatasmithMaterialExpressionScalar&>(InExpression);
			OutValue.Set(ScalarExpression.GetScalar());
		}
		else if (InExpression.IsSubType(EDatasmithMaterialExpressionType::FlattenNormal))
		{
			const IDatasmithMaterialExpressionFlattenNormal& NormalExpression = static_cast<const IDatasmithMaterialExpressionFlattenNormal&>(InExpression);
			FDatasmithInputValue Normal(NormalExpression.GetNormal());
			FDatasmithInputValue Flatness(NormalExpression.GetFlatness());

			OutValue = MoveTemp(Normal); // TODO: Evaluate FlattenNormal
			if (!OutValue.bNumericValid && Flatness.bNumericValid)
			{
				OutValue.Set(Flatness.Numeric);
			}
		}
		else if (InExpression.IsSubType(EDatasmithMaterialExpressionType::FunctionCall))
		{
			const IDatasmithMaterialExpressionFunctionCall& FctCallExpression = static_cast<const IDatasmithMaterialExpressionFunctionCall&>(InExpression);
			EvalExpressionFctCall(FctCallExpression, OutValue);
		}
		else if (InExpression.IsSubType(EDatasmithMaterialExpressionType::Generic))
		{
			const IDatasmithMaterialExpressionGeneric& GenericExpression = static_cast<const IDatasmithMaterialExpressionGeneric&>(InExpression);
			EvalExpressionGeneric(GenericExpression, OutValue);
		}
		else if (InExpression.IsSubType(EDatasmithMaterialExpressionType::Texture))
		{
			const IDatasmithMaterialExpressionTexture& TextureExpression = static_cast<const IDatasmithMaterialExpressionTexture&>(InExpression);
			OutValue.Set(TextureExpression.GetTexturePathName());
			EvaluateInput(TextureExpression.GetInputCoordinate(), OutValue);
		}
		else if (InExpression.IsSubType(EDatasmithMaterialExpressionType::TextureCoordinate))
		{
			const IDatasmithMaterialExpressionTextureCoordinate& CoordinateExpression = static_cast<const IDatasmithMaterialExpressionTextureCoordinate&>(InExpression);
			if (!OutValue.Texture)
			{
				OutValue.Set(TEXT(""));
			}
			OutValue.Texture->CoordinateIndex = CoordinateExpression.GetCoordinateIndex();
			OutValue.Texture->UTiling = CoordinateExpression.GetUTiling();
			OutValue.Texture->VTiling = CoordinateExpression.GetVTiling();
		}
		else
		{
			OutValue.bNumericValid = false;
		}
	}


	// Evaluate is a basic Function (multiply, add, ...)
	void FDatasmithExpressionEvaluator::EvalExpressionGeneric(const IDatasmithMaterialExpressionGeneric& InGenericExpression, FDatasmithInputValue& OutValue)
	{
		bool bInvalid = false;
		const TCHAR* Expression = InGenericExpression.GetExpressionName();

		if (InGenericExpression.GetInputCount() >= 3)
		{
			if (FCString::Stricmp(Expression, TEXT("LinearInterpolate")) == 0)
			{
				EvaluateInput(InGenericExpression.GetInput(0), OutValue);
				FDatasmithInputValue  InputB;
				EvaluateInput(InGenericExpression.GetInput(1), InputB);
				FDatasmithInputValue  InputAlpha;
				EvaluateInput(InGenericExpression.GetInput(2), InputAlpha);

				// If Texture is on the second node, make it first
				if (!OutValue.Texture && InputB.Texture)
				{
					Swap(InputB, OutValue);
					if (InputAlpha.bNumericValid)
					{
						InputAlpha.Set(1.0f - InputAlpha.GetScalar());
					}
				}

				if (!InputAlpha.bNumericValid)
				{
					return;
				}

				if (OutValue.bNumericValid && InputB.bNumericValid)
				{
					OutValue.Set(FMath::Lerp(OutValue.Numeric, InputB.Numeric, InputAlpha.GetScalar()));
				}
				else if (OutValue.Texture)
				{
					OutValue.Texture->Fading = InputAlpha.GetScalar();
				}
				return;
			}
			if (FCString::Stricmp(Expression, TEXT("StaticSwitch")) == 0)
			{
				FDatasmithInputValue  InputToggle;
				EvaluateInput(InGenericExpression.GetInput(2), InputToggle);
				if (!InputToggle.bNumericValid || InputToggle.GetScalar() > 0.5)
				{
					EvaluateInput(InGenericExpression.GetInput(0), OutValue);
				}
				else
				{
					EvaluateInput(InGenericExpression.GetInput(1), OutValue);
				}
				return;
			}
		}
		if (FCString::Stricmp(Expression, TEXT("TextureObjectParameter")) == 0)
		{
			const TSharedPtr< IDatasmithKeyValueProperty >& TextureProperty = InGenericExpression.GetPropertyByName(TEXT("Texture"));
			if (TextureProperty.IsValid())
			{
				OutValue.Set(TextureProperty->GetValue());
			}
		}
		else if (FCString::Stricmp(Expression, TEXT("Fresnel")) == 0)
		{
			OutValue.Set(0.1f); // Not supported
		}
		else if (FCString::Stricmp(Expression, TEXT("VertexNormalWS")) == 0)
		{
			OutValue.Set(1.0f); // Not supported
		}
		else if (InGenericExpression.GetInputCount() >= 1)
		{
			EvaluateInput(InGenericExpression.GetInput(0), OutValue);
			if (!OutValue.bNumericValid)
			{
				return;
			}

			if (FCString::Stricmp(Expression, TEXT("OneMinus")) == 0)
			{
				OutValue.Set(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f) - OutValue.Numeric);
			}
			else if (FCString::Stricmp(Expression, TEXT("Desaturation")) == 0)
			{
				OutValue.Numeric = OutValue.Numeric.Desaturate(1.0);
			}
			else if (FCString::Stricmp(Expression, TEXT("Noise")) == 0)
			{
				OutValue.Set(0.5); // Not supported
			}
			else if (InGenericExpression.GetInputCount() == 1 && FCString::Stricmp(Expression, TEXT("Power")) == 0)
			{
				// Power with a scalar component
				const TSharedPtr< IDatasmithKeyValueProperty >& ExponentProperty = InGenericExpression.GetPropertyByName(TEXT("ConstExponent"));
				if (!ExponentProperty.IsValid())
				{
					OutValue.bNumericValid = false;
					return;
				}
				float Exponent = FCString::Atof(ExponentProperty->GetValue());
				FLinearColor PowValue1(FPlatformMath::Pow(OutValue.Numeric.R, Exponent),
					FPlatformMath::Pow(OutValue.Numeric.G, Exponent),
					FPlatformMath::Pow(OutValue.Numeric.B, Exponent),
					FPlatformMath::Pow(OutValue.Numeric.A, Exponent));
				OutValue.Set(PowValue1);
			}
			else if (InGenericExpression.GetInputCount() >= 2)
			{
				FDatasmithInputValue  Input1;
				EvaluateInput(InGenericExpression.GetInput(1), Input1);
				if (!Input1.bNumericValid)
				{
					return;
				}

				if (FCString::Stricmp(Expression, TEXT("Add")) == 0)
				{
					OutValue.Numeric += Input1.Numeric;
				}
				else if (FCString::Stricmp(Expression, TEXT("Multiply")) == 0)
				{
					OutValue.Numeric *= Input1.Numeric;
				}
				else if (FCString::Stricmp(Expression, TEXT("Power")) == 0)
				{
					FLinearColor    PowValue(FPlatformMath::Pow(OutValue.Numeric.R, Input1.Numeric.R),
						FPlatformMath::Pow(OutValue.Numeric.G, Input1.Numeric.G),
						FPlatformMath::Pow(OutValue.Numeric.B, Input1.Numeric.B),
						FPlatformMath::Pow(OutValue.Numeric.A, Input1.Numeric.A));
					OutValue.Set(PowValue);
				}
				else if (FCString::Stricmp(Expression, TEXT("Max")) == 0)
				{
					FLinearColor    MaxValue(FMath::Max(OutValue.Numeric.R, Input1.Numeric.R),
						FMath::Max(OutValue.Numeric.G, Input1.Numeric.G),
						FMath::Max(OutValue.Numeric.B, Input1.Numeric.B),
						FMath::Max(OutValue.Numeric.A, Input1.Numeric.A));
					OutValue.Set(MaxValue);
				}
				else if (FCString::Stricmp(Expression, TEXT("AppendVector")) == 0)
				{
					FLinearColor    Constant2VectorValue(OutValue.Numeric.R, Input1.Numeric.R, 0, 0);
					OutValue.Set(Constant2VectorValue);
				}
				else if (InGenericExpression.GetInputCount() >= 3)
				{
					FDatasmithInputValue  Input2;
					EvaluateInput(*InGenericExpression.GetInput(2), Input2);
					if (!Input2.bNumericValid)
					{
						OutValue.bNumericValid = false;
						return;
					}

					bInvalid = true;
				}
				else
				{
					bInvalid = true;
				}
			}
			else
			{
				bInvalid = true;
			}
		}
		else
		{
			bInvalid = true;
		}

		if (bInvalid)
		{
			OutValue.bNumericValid = false;
		}
	}


	// Evaluate is a Unreal Engine Function
	void FDatasmithExpressionEvaluator::EvalExpressionFctCall(const IDatasmithMaterialExpressionFunctionCall& InFctCallExpression, FDatasmithInputValue& OutValue)
	{
		const TCHAR* FctPathName = InFctCallExpression.GetFunctionPathName();
		int32 InputCount = InFctCallExpression.GetInputCount();
		if (InputCount == 0)
		{
			if (FCString::Strstr(FctPathName, TEXT("LocalPosition")))
			{
				OutValue.Set(FLinearColor(1.0, 2.0, 3.0, 1.0)); // Not really supported
				return;
			}

			return;
		}

		TArray<FDatasmithInputValue> Inputs;
		Inputs.SetNum(InputCount);

		Inputs[0] = MoveTemp(OutValue); // Copy first input result in output
		EvalExpressionChildren(InFctCallExpression, &Inputs[0]);
		OutValue = MoveTemp(Inputs[0]); // Copy first input result in output

		if (FCString::Strstr(FctPathName, TEXT("UVEdit.UVEdit")))
		{
			FDatasmithInputValue::FTexture* Texture = OutValue.Texture.Get();
			if (Texture == nullptr)
			{
				OutValue.bNumericValid = false;
				return;
			}

			// Input 0 must be a TextureCoordinate so Index, UTiling and VTiling are already set.

			if (InputCount > 1 && Inputs[1].bNumericValid)
			{
				Texture->UTilingPivot = Inputs[1].Numeric.Component(0);
				Texture->VTilingPivot = Inputs[1].Numeric.Component(1);
			}
			if (InputCount > 2 && Inputs[2].bNumericValid)
			{
				Texture->UTiling = Inputs[2].Numeric.Component(0);
				Texture->VTiling = Inputs[2].Numeric.Component(1);
			}
			if (InputCount > 3 && Inputs[3].bNumericValid)
			{
				Texture->bMirrorU = Inputs[3].GetScalar() != 0.0f;
			}
			if (InputCount > 4 && Inputs[4].bNumericValid)
			{
				Texture->bMirrorV = Inputs[4].GetScalar() != 0.0f;
			}
			if (InputCount > 5 && Inputs[5].bNumericValid)
			{
				Texture->URotationPivot = Inputs[5].Numeric.Component(0);
				Texture->VRotationPivot = Inputs[5].Numeric.Component(1);
			}
			if (InputCount > 6 && Inputs[6].bNumericValid)
			{
				Texture->Rotation = Inputs[6].GetScalar();
			}
			if (InputCount > 7 && Inputs[7].bNumericValid)
			{
				Texture->UOffset = Inputs[7].Numeric.Component(0);
				Texture->VOffset = Inputs[7].Numeric.Component(1);
			}
		}
		else if (FCString::Strstr(FctPathName, TEXT("FlattenNormal")))
		{
			if (InputCount > 1 && Inputs[1].bNumericValid)
			{
				if (!OutValue.bNumericValid)
				{
					OutValue.Set(Inputs[1].Numeric);
				}
			}
		}
		else if (FCString::Strstr(FctPathName, TEXT("NormalFromHeightmap.NormalFromHeightmap")))
		{
			if (InputCount > 1 && Inputs[1].bNumericValid) // Unused
			{
			}
			if (InputCount > 2 && Inputs[2].bNumericValid) // UVOffset
			{
				if (!OutValue.Texture)
				{
					OutValue.Set(TEXT(""));
				}
				OutValue.Texture->UVOffset = Inputs[2].GetScalar();
			}
			if (InputCount > 3 && Inputs[3].bNumericValid) // UVCoordinatesInput
			{
				if (Inputs[3].Texture)
				{
					if (OutValue.Texture)
					{
						OutValue.Texture->CoordinateIndex = Inputs[3].Texture->CoordinateIndex;
						OutValue.Texture->UTiling = Inputs[3].Texture->UTiling;
						OutValue.Texture->VTiling = Inputs[3].Texture->VTiling;
					}
					else
					{
						OutValue.Texture = MoveTemp(Inputs[3].Texture);
					}
				}
			}
		}
		else if (FCString::Strstr(FctPathName, TEXT("ConvertFromDiffSpec.ConvertFromDiffSpec")))
		{
			if (OutValue.bNumericValid)
			{
				if (InputCount > 1 && Inputs[1].bNumericValid) // Unused
				{
					// Simple approximation
					OutValue.Set(FMath::Lerp(OutValue.Numeric, OutValue.Numeric + Inputs[1].Numeric, 0.5));
				}
				else
					OutValue.bNumericValid = false;
			}
		}
		else
		{
			OutValue.bNumericValid = false;
		}
	}


	void FDatasmithExpressionEvaluator::EvalExpressionChildren(const IDatasmithMaterialExpression& InExpression, FDatasmithInputValue* InOutInputs)
	{
		int32 InputCount = InExpression.GetInputCount();
		for (int32 Index = 0; Index < InputCount; ++Index)
		{
			EvaluateInput(InExpression.GetInput(Index), InOutInputs[Index]);
		}
	}


	// Evaluate this expression
	void FDatasmithInputValue::EvaluateInput(const IDatasmithExpressionInput& InInput)
	{
		FDatasmithExpressionEvaluator::EvaluateInput(InInput, *this);
	}
}
