// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Templates/SharedPointer.h"

namespace DatasmithRuntime { class FSceneImporter; }

class IDatasmithMaterialInstanceElement;
class IDatasmithUEPbrMaterialElement;
class UMaterialInstanceDynamic;

namespace DatasmithRuntime
{
	extern const TCHAR* MATERIAL_HOST;

	namespace EMaterialRequirements
	{
		enum Type
		{
			RequiresNothing = 0x00,
			RequiresNormals = 0x01,
			RequiresTangents = 0x02,
			RequiresAdjacency = 0x04,
		};
	};

	extern FName PbrTexturePropertyNames[];

	using FTextureCallback = TFunction<void(const FString&, int32)>;

	TSharedPtr<IDatasmithUEPbrMaterialElement> ValidatePbrMaterial(TSharedPtr<IDatasmithUEPbrMaterialElement> PbrMaterialElement, FSceneImporter& SceneImporter);

	int32 ProcessMaterialElement(TSharedPtr< IDatasmithMaterialInstanceElement > BaseMaterialElement, FTextureCallback TextureCallback);

	int32 ProcessMaterialElement(IDatasmithUEPbrMaterialElement* PbrMaterialElement, FTextureCallback TextureCallback);

	bool LoadReferenceMaterial(UMaterialInstanceDynamic* MaterialInstance, TSharedPtr<IDatasmithMaterialInstanceElement>& MaterialElement);

	bool LoadPbrMaterial(IDatasmithUEPbrMaterialElement& UEPbrMaterial, UMaterialInstanceDynamic* MaterialInstance);
}

