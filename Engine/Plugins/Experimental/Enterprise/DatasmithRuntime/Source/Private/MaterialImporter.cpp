// Copyright Epic Games, Inc. All Rights Reserved.


#include "DatasmithRuntimeUtils.h"
#include "LogCategory.h"
#include "MaterialImportUtils.h"

#include "DatasmithMaterialElements.h"
#include "DatasmithUtils.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"

#include "Engine/TextureLightProfile.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MaterialShared.h"

namespace DatasmithRuntime
{
	void UpdateMaterials(TSet<FSceneGraphId>& MaterialElementSet, TMap< FSceneGraphId, FAssetData >& AssetDataList);

	void FSceneImporter::ProcessMaterialData(FAssetData& MaterialData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::ProcessMaterialData);

		// Something is wrong. Do not go any further
		if (MaterialData.HasState(EAssetState::PendingDelete))
		{
			UE_LOGF(LogDatasmithRuntime, Error, "A material marked for deletion is actually used by the scene");
			return;
		}

		if (MaterialData.HasState(EAssetState::Processed))
		{
			return;
		}

		TSharedPtr< IDatasmithElement >& Element = Elements[ MaterialData.ElementId ];

		bool bUsingMaterialFromCache = false;

		if ( !MaterialData.Object.IsValid() )
		{
			MaterialData.Hash = GetTypeHash(Element->CalculateElementHash(true), EDataType::Material);

			if (UObject* Asset = FAssetRegistry::FindObjectFromHash(MaterialData.Hash))
			{
				UMaterialInstanceDynamic* Material = Cast<UMaterialInstanceDynamic>(Asset);
				check(Material);

				MaterialData.Object = TWeakObjectPtr<UObject>(Material);

				bUsingMaterialFromCache = true;
			}
			else
			{
#ifdef ASSET_DEBUG
				UPackage* Package = CreatePackage(*FPaths::Combine( TEXT("/Game/Runtime/Materials"), MaterialName));
				MaterialData.Object = TWeakObjectPtr<UObject>( UMaterialInstanceDynamic::Create( nullptr, Package) );
				MaterialData.Object->SetFlags(RF_Public);
#else
				MaterialData.Object = TWeakObjectPtr<UObject>( UMaterialInstanceDynamic::Create( nullptr, nullptr) );
#endif
				check(MaterialData.Object.IsValid());

				FString MaterialName = FString::Printf(TEXT("M_%s_%d"), Element->GetName(), MaterialData.ElementId);
				MaterialName = FDatasmithUtils::SanitizeObjectName(MaterialName);
				RenameObject(MaterialData.GetObject(), *MaterialName);

				// Load metadata on newly created material asset if any
				ApplyMetadata(MaterialData.MetadataId, MaterialData.GetObject());
			}
		}

		if ( Element->IsA( EDatasmithElementType::UEPbrMaterial ) )
		{
			Element = ValidatePbrMaterial(StaticCastSharedPtr< IDatasmithUEPbrMaterialElement >(Element), *this);
		}

		FActionTaskFunction AssignTextureFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
		{
			if (UTextureLightProfile* TextureProfile = Cast<UTextureLightProfile>(Object))
			{
				return this->AssignProfileTexture(Referencer, TextureProfile);
			}

			return this->AssignTexture(Referencer, Cast<UTexture2D>(Object));
		};

		FTextureCallback TextureCallback;
		TextureCallback = [this, ElementId = MaterialData.ElementId, AssignTextureFunc](const FString& TextureNamePrefixed, int32 PropertyIndex)->void
		{
			if (FSceneGraphId* ElementIdPtr = this->AssetElementMapping.Find(TextureNamePrefixed))
			{
				this->ProcessTextureData(*ElementIdPtr);

				this->AddToQueue(EQueueTask::NonAsyncQueue, { AssignTextureFunc, *ElementIdPtr, { EDataType::Material, ElementId, (uint16)PropertyIndex } });
			}
		};

		const FString Host = FDatasmithReferenceMaterialManager::Get().GetHostFromString( SceneElement->GetHost() );

		if( Element->IsA( EDatasmithElementType::UEPbrMaterial ) )
		{
			MaterialData.Requirements = ProcessMaterialElement(static_cast<IDatasmithUEPbrMaterialElement*>(Element.Get()), TextureCallback);
		}
		else if( Element->IsA( EDatasmithElementType::MaterialInstance ) )
		{
			MaterialData.Requirements = ProcessMaterialElement(StaticCastSharedPtr<IDatasmithMaterialInstanceElement>(Element), TextureCallback);
		}

		MaterialData.SetState(EAssetState::Processed);

		FAssetRegistry::RegisterAssetData(MaterialData.GetObject<>(), SceneKey, MaterialData);

		if (!bUsingMaterialFromCache)
		{
			FActionTaskFunction TaskFunc = [this](UObject*, const FReferencer& Referencer) -> EActionResult::Type
			{
				return this->ProcessMaterial(Referencer.GetId());
			};

			AddToQueue(EQueueTask::MaterialQueue, { TaskFunc, {EDataType::Material, MaterialData.ElementId, 0 } });
			TasksToComplete |= EWorkerTask::MaterialCreate;

			MaterialElementSet.Add(MaterialData.ElementId);
		}
		else if(FAssetRegistry::IsObjectCompleted(MaterialData.GetObject<>()))
		{
			MaterialData.AddState(EAssetState::Completed);
		}
	}

	EActionResult::Type FSceneImporter::ProcessMaterial(FSceneGraphId ElementId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::ProcessMaterial);

		FAssetData& MaterialData = AssetDataList[ ElementId ];

		TSharedPtr< IDatasmithElement >& Element = Elements[ ElementId ];

		UMaterialInstanceDynamic* MaterialInstance = MaterialData.GetObject<UMaterialInstanceDynamic>();

		bool bCreationSuccessful = false;

		if ( Element->IsA( EDatasmithElementType::Material ) )
		{
			// Not supported
		}
		else if ( Element->IsA( EDatasmithElementType::MaterialInstance ) )
		{

			TSharedPtr< IDatasmithMaterialInstanceElement > MaterialElement = StaticCastSharedPtr< IDatasmithMaterialInstanceElement >( Element );

			bCreationSuccessful = LoadReferenceMaterial(MaterialInstance, MaterialElement);

			// Add tracking on material's properties
			if (bCreationSuccessful)
			{
				for (int Index = 0; Index < MaterialElement->GetPropertiesCount(); ++Index)
				{
					const TSharedPtr< IDatasmithKeyValueProperty >& Property = MaterialElement->GetProperty(Index);
					DependencyList.Add(Property->GetNodeId(), { EDataType::Material, ElementId, 0xffff });
				}
			}
		}
		else if ( Element->IsA( EDatasmithElementType::UEPbrMaterial ) )
		{
			bCreationSuccessful = LoadPbrMaterial(static_cast<IDatasmithUEPbrMaterialElement&>( *Element ), MaterialInstance);
		}

		ActionCounter.Increment();

		if (bCreationSuccessful)
		{
			FAssetRegistry::SetObjectCompletion(MaterialInstance, true);
		}
		else
		{
			FAssetRegistry::UnregisteredAssetsData(MaterialInstance, 0, [](FAssetData& AssetData) -> void
				{
					AssetData.AddState(EAssetState::Completed);
					AssetData.Object.Reset();
				});

			UE_LOGF(LogDatasmithRuntime, Warning, "Failed to create material %ls", Element->GetName());

			return EActionResult::Failed;
		}

		return EActionResult::Succeeded;
	}

	EActionResult::Type FSceneImporter::AssignTexture(const FReferencer& Referencer, UTexture2D* Texture)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::AssignTexture);

		if (Texture)
		{
			const FSceneGraphId ElementId = Referencer.GetId();

			FAssetData& MaterialData = AssetDataList[ ElementId ];

			if (!MaterialData.HasState(EAssetState::Completed))
			{
				return EActionResult::Retry;
			}

			UMaterialInstanceDynamic* MaterialInstance = MaterialData.GetObject<UMaterialInstanceDynamic>();
			if (!MaterialInstance)
			{
				return EActionResult::Failed;
			}

			TSharedPtr< IDatasmithElement >& Element = Elements[ ElementId ];

			if ( Element->IsA( EDatasmithElementType::Material ) )
			{
				// Not supported
			}
			else if ( Element->IsA( EDatasmithElementType::MaterialInstance ) )
			{
				IDatasmithMaterialInstanceElement* MaterialElement = static_cast< IDatasmithMaterialInstanceElement* >( Element.Get() );

				const int32 PropertyIndex = Referencer.Slot;
				const TSharedPtr< IDatasmithKeyValueProperty > Property = MaterialElement->GetProperty(PropertyIndex);
				ensure(Property.IsValid());

				FName PropertyName(Property->GetName());
				MaterialInstance->SetTextureParameterValue(PropertyName, Texture);
#ifdef ASSET_DEBUG
				Texture->ClearFlags(RF_Public);
#endif
			}
			else if ( Element->IsA( EDatasmithElementType::UEPbrMaterial ) )
			{
				FName PropertyName(PbrTexturePropertyNames[Referencer.Slot]);
				MaterialInstance->SetTextureParameterValue(PropertyName, Texture);
#ifdef ASSET_DEBUG
				Texture->ClearFlags(RF_Public);
#endif
			}
		}

		ActionCounter.Increment();

		return EActionResult::Succeeded;
	}

	void UpdateMaterials(TSet<FSceneGraphId>& MaterialElementSet, TMap< FSceneGraphId, FAssetData >& AssetDataList)
	{
		FMaterialUpdateContext MaterialUpdateContext;

		for( FSceneGraphId ElementId : MaterialElementSet )
		{
			FAssetData& MaterialData = AssetDataList[ ElementId ];

			if (UMaterialInstanceDynamic* MaterialInstance = MaterialData.GetObject<UMaterialInstanceDynamic>())
			{
				MaterialUpdateContext.AddMaterialInstance( MaterialInstance );

//#if WITH_EDITOR
//				// If BlendMode override property has been changed, make sure this combination of the parent material is compiled
//				if ( MaterialInstance->BasePropertyOverrides.bOverride_BlendMode == true )
//				{
//					MaterialInstance->ForceRecompileForRendering();
//				}
//				else
//				{
//					// If a switch is overridden, we need to recompile
//					FStaticParameterSet StaticParameters;
//					MaterialInstance->GetStaticParameterValues( StaticParameters );
//
//					for ( FStaticSwitchParameter& Switch : StaticParameters.StaticSwitchParameters )
//					{
//						if ( Switch.bOverride )
//						{
//							MaterialInstance->ForceRecompileForRendering();
//							break;
//						}
//					}
//				}
//
//				MaterialInstance->PreEditChange( nullptr );
//				MaterialInstance->PostEditChange();
//#endif
			}
		}
	}
} // End of namespace DatasmithRuntime
