// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithRuntimeMaterialSelector.h"

#include "DatasmithSceneFactory.h"

#include "UObject/SoftObjectPath.h"

FDatasmithRuntimeMaterialSelector::FDatasmithRuntimeMaterialSelector()
{
	OpaqueMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithRuntime/Materials/M_Opaque.M_Opaque") );
	TransparentMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithRuntime/Materials/M_Transparent.M_Transparent") );
	CutoutMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithRuntime/Materials/M_Cutout.M_Cutout") );
}

bool FDatasmithRuntimeMaterialSelector::IsValid() const
{
	return OpaqueMaterial.IsValid() && TransparentMaterial.IsValid() && CutoutMaterial.IsValid();
}

const FDatasmithReferenceMaterial& FDatasmithRuntimeMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const
{
	TSharedPtr< IDatasmithMaterialInstanceElement > MaterialElement = ConstCastSharedPtr< IDatasmithMaterialInstanceElement >(InDatasmithMaterial);

	TFunction<void(const TCHAR*, const TCHAR*)> ConvertProperty;
	ConvertProperty = [this, &MaterialElement](const TCHAR* BoolPropertyName, const TCHAR* FloatPropertyName)
	{
		TSharedPtr< IDatasmithKeyValueProperty > BoolProperty = MaterialElement->GetPropertyByName(BoolPropertyName);
		if (BoolProperty.IsValid())
		{
			bool Value;
			this->GetBool(BoolProperty, Value);
			if (Value)
			{
				TSharedPtr<IDatasmithKeyValueProperty> NewProperty = MaterialElement->GetPropertyByName(FloatPropertyName);
				if (!NewProperty.IsValid())
				{
					NewProperty = FDatasmithSceneFactory::CreateKeyValueProperty(FloatPropertyName);
					MaterialElement->AddProperty(NewProperty);
				}
				NewProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
				NewProperty->SetValue(TEXT("1.0"));
			}
		}
	};

	// Convert glossiness into roughness' equivalent
	TSharedPtr< IDatasmithKeyValueProperty > Glossiness = MaterialElement->GetPropertyByName(TEXT("Glossiness"));
	if (Glossiness.IsValid())
	{
		TSharedPtr<IDatasmithKeyValueProperty> Roughness = MaterialElement->GetPropertyByName(TEXT("Roughness"));
		if (!Roughness.IsValid())
		{
			Roughness = FDatasmithSceneFactory::CreateKeyValueProperty(TEXT("Roughness"));
			Roughness->SetPropertyType(EDatasmithKeyValuePropertyType::Float);

			MaterialElement->AddProperty(Roughness);
		}

		float Value;
		GetFloat(Glossiness, Value);
		FString NewValue = FString::Printf(TEXT("%f"), 1.f - Value);
		Roughness->SetValue(*NewValue);
	}

	// Convert static boolean parameters into float ones used in reference material's graph
	ConvertProperty(TEXT("RoughnessMapEnable"), TEXT("RoughnessMapFading"));
	ConvertProperty(TEXT("IsMetal"), TEXT("Metallic"));
	ConvertProperty(TEXT("TintEnabled"), TEXT("TintColorFading"));
	ConvertProperty(TEXT("SelfIlluminationMapEnable"), TEXT("SelfIlluminationMapFading"));
	ConvertProperty(TEXT("IsPbr"), TEXT("UseNormalMap"));

	// Return proper material based on material's type
	if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::Transparent)
	{
		return TransparentMaterial;
	}
	else if (InDatasmithMaterial->GetMaterialType() == EDatasmithReferenceMaterialType::CutOut)
	{
		return  CutoutMaterial;
	}

	return OpaqueMaterial;
}

void FDatasmithRuntimeMaterialSelector::FinalizeMaterialInstance(const TSharedPtr<IDatasmithMaterialInstanceElement>& InDatasmithMaterial, UMaterialInstanceConstant * MaterialInstance) const
{
	// Nothing to do there.
}
