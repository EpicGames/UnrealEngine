// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"


class IDatasmithMaterialInstanceElement;

class FDatasmithRuntimeMaterialSelector : public FDatasmithReferenceMaterialSelector
{
public:
	FDatasmithRuntimeMaterialSelector();
	virtual ~FDatasmithRuntimeMaterialSelector() = default;

	virtual bool IsValid() const override;
	virtual const FDatasmithReferenceMaterial& GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& /*InDatasmithMaterial*/ ) const override;
	virtual void FinalizeMaterialInstance(const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial, UMaterialInstanceConstant* MaterialInstance) const override;

private:
	FDatasmithReferenceMaterial OpaqueMaterial;
	FDatasmithReferenceMaterial TransparentMaterial;
	FDatasmithReferenceMaterial CutoutMaterial;
};
