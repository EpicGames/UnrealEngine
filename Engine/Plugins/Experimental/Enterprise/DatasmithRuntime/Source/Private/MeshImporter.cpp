// Copyright Epic Games, Inc. All Rights Reserved.


#include "DatasmithRuntimeUtils.h"
#include "DatasmithTranslator.h"
#include "LogCategory.h"

#include "DatasmithPayload.h"
#include "DatasmithUtils.h"
#include "Engine/CollisionProfile.h"
#include "IDatasmithSceneElements.h"
#include "Engine/StaticMeshSourceData.h"
#include "StaticMeshResources.h"
#include "Misc/Paths.h"
#include "Utility/DatasmithMeshHelper.h"

#include "Async/Async.h"
#include "Components/StaticMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/StaticMeshActor.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#endif

namespace DatasmithRuntime
{
	bool FSceneImporter::ProcessMeshData(FAssetData& MeshData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::ProcessMeshData);

		// Something is wrong. Do not go any further
		if (MeshData.HasState(EAssetState::PendingDelete))
		{
			UE_LOGF(LogDatasmithRuntime, Error, "A mesh marked for deletion is actually used by the scene");
			return false;
		}

		if (MeshData.HasState(EAssetState::Processed))
		{
			return true;
		}

		TSharedPtr< IDatasmithMeshElement > MeshElement = StaticCastSharedPtr< IDatasmithMeshElement >(Elements[MeshData.ElementId]);

		URuntimeMesh* StaticMesh = MeshData.GetObject<URuntimeMesh>();

		// If static mesh already completed, check if geometry has changed
		if (StaticMesh)
		{
			uint32 NewResourceHash = GetTypeHash(MeshElement->GetFileHash());
			// Force recreation of the static mesh if the mesh's file has changed
			if (MeshData.ResourceHash != NewResourceHash)
			{
				MeshData.ClearState(EAssetState::Completed);
				FAssetRegistry::UnregisterAssetData(StaticMesh, SceneKey, MeshData.ElementId);
				StaticMesh = nullptr;
				MeshData.Object.Reset();
			}
		}

		if (StaticMesh == nullptr)
		{
			MeshData.Hash = GetTypeHash(MeshElement->CalculateElementHash(true), EDataType::Mesh);
			MeshData.ResourceHash = GetTypeHash(MeshElement->GetFileHash());

			if (UObject* AssetPtr = FAssetRegistry::FindObjectFromHash(MeshData.Hash))
			{
				StaticMesh = Cast<URuntimeMesh>(AssetPtr);
				check(StaticMesh);
			}
			else
			{
				FString MeshName = FString::Printf(TEXT("SM_%s_%d"), MeshElement->GetName(), MeshData.ElementId);
				MeshName = FDatasmithUtils::SanitizeObjectName(MeshName);
#ifdef ASSET_DEBUG
				UPackage* Package = CreatePackage(*FPaths::Combine( TEXT("/Game/Runtime/Meshes"), MeshName));
				StaticMesh = NewObject< URuntimeMesh >(Package, NAME_None, RF_Public);
#else
				StaticMesh = NewObject< URuntimeMesh >(GetTransientPackage());
#endif
				check(StaticMesh);

				RenameObject(StaticMesh, *MeshName);

				StaticMesh->SetWorld(RootComponent->GetWorld());

				// Add the creation of the mesh to the queue
				FActionTaskFunction TaskFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
				{
					this->OnGoingTasks.Emplace( Async(
#if WITH_EDITOR
						EAsyncExecution::LargeThreadPool,
#else
						EAsyncExecution::ThreadPool,
#endif
						[this, ElementId = Referencer.GetId()]() -> bool
						{
							return this->CreateStaticMesh(ElementId);
						},
						[this]()->void
						{
							this->ActionCounter.Increment();
						}
					));

					return EActionResult::Succeeded;
				};

				AddToQueue(EQueueTask::MeshQueue, { TaskFunc, {EDataType::Mesh, MeshData.ElementId, 0 } });
				TasksToComplete |= EWorkerTask::MeshCreate;

				// If applicable, Apply metadata on newly created static mesh
				ApplyMetadata(MeshData.MetadataId, StaticMesh);

				MeshElementSet.Add(MeshData.ElementId);
			}

			MeshData.Object = TWeakObjectPtr< UObject >(StaticMesh);
		}

		// Collect materials used by static mesh
		for (int32 Index = 0; Index < MeshElement->GetMaterialSlotCount(); Index++)
		{
			if (const IDatasmithMaterialIDElement* MaterialIDElement = MeshElement->GetMaterialSlotAt(Index).Get())
			{
				const FString MaterialPathName(MaterialIDElement->GetName());

				if (!MaterialPathName.StartsWith(TEXT("/")))
				{
					if (FSceneGraphId* MaterialElementIdPtr = AssetElementMapping.Find(MaterialPrefix + MaterialPathName))
					{
						ProcessMaterialData(AssetDataList[*MaterialElementIdPtr]);
					}
				}
				else
				{
					// Force loading of material asset if it exists. It will be assigned when the mesh is built
					UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(FSoftObjectPath(MaterialPathName).TryLoad());
					if (MaterialInterface == nullptr)
					{
						UE_LOGF(LogDatasmithRuntime, Warning, "ProcessMeshData: Cannot find material %ls", *MaterialPathName);
					}
				}
			}
		}

		// If static mesh already completed, then material assignment has changed
		if (MeshData.HasState(EAssetState::Completed))
		{
			UpdateStaticMeshMaterials(MeshData);
		}

		// Create BodySetup in game thread to avoid allocating during a garbage collect later on
		if (StaticMesh->GetBodySetup() == nullptr)
		{
			StaticMesh->CreateBodySetup();
		}

		MeshData.SetState(EAssetState::Processed);

		FAssetRegistry::RegisterAssetData(StaticMesh, SceneKey, MeshData);

		return true;
	}

	bool FSceneImporter::ProcessMeshActorData(FActorData& ActorData, IDatasmithMeshActorElement* MeshActorElement)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::ProcessMeshActorData);

		if (ActorData.HasState(EAssetState::Processed))
		{
			return true;
		}

		// Invalid reference to a mesh. Abort creation of component
		if (FCString::Strlen(MeshActorElement->GetStaticMeshPathName()) == 0)
		{
			ActorData.SetState(EAssetState::Processed);
			return false;
		}

		FActionTaskFunction CreateComponentFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
		{
			return this->CreateMeshComponent(Referencer.GetId(), Cast<UStaticMesh>(Object));
		};

		FString StaticMeshPathName(MeshActorElement->GetStaticMeshPathName());
		UStaticMesh* StaticMesh = nullptr;

		if (!StaticMeshPathName.StartsWith(TEXT("/")))
		{
			if (FSceneGraphId* MeshElementIdPtr = AssetElementMapping.Find(MeshPrefix + StaticMeshPathName))
			{
				FAssetData& MeshData = AssetDataList[*MeshElementIdPtr];

				if (!ProcessMeshData(MeshData))
				{
					return false;
				}

				AddToQueue(EQueueTask::NonAsyncQueue, { CreateComponentFunc, *MeshElementIdPtr, { EDataType::Actor, ActorData.ElementId, 0 } });
				TasksToComplete |= EWorkerTask::MeshComponentCreate;

				ActorData.AssetId = *MeshElementIdPtr;

				StaticMesh = MeshData.GetObject<UStaticMesh>();
			}
		}
		else
		{
			StaticMesh = Cast<UStaticMesh>(FSoftObjectPath(StaticMeshPathName).TryLoad());
		}

		// The referenced static mesh was not found. Abort creation of component
		if (StaticMesh == nullptr)
		{
			return false;
		}

		if (MeshActorElement->GetMaterialOverridesCount() > 0)
		{
			FActionTaskFunction AssignMaterialFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
			{
				return this->AssignMaterial(Referencer, Cast<UMaterialInstanceDynamic>(Object));
			};

			TArray< FStaticMaterial >& StaticMaterials = StaticMesh->GetStaticMaterials();

			TMap<FString, int32> SlotMapping;
			SlotMapping.Reserve(StaticMaterials.Num());

			for (int32 Index = 0; Index < StaticMaterials.Num(); ++Index)
			{
				const FStaticMaterial& StaticMaterial = StaticMaterials[Index];

				if (StaticMaterial.MaterialSlotName != NAME_None)
				{
					SlotMapping.Add(StaticMaterial.MaterialSlotName.ToString(), Index);
				}
			}

			// #ue_datasmithruntime: Missing code to handle the case where a MaterialID's name is an asset's path

			// All the materials of the static mesh are overridden by one single material
			// Note: for that case, we assume the actor has only one override
			if (MeshActorElement->GetMaterialOverride(0)->GetId() == -1)
			{
				TSharedPtr<const IDatasmithMaterialIDElement> MaterialIDElement = MeshActorElement->GetMaterialOverride(0);

				if (FSceneGraphId* MaterialElementIdPtr = AssetElementMapping.Find(MaterialPrefix + MaterialIDElement->GetName()))
				{
					ProcessMaterialData(AssetDataList[*MaterialElementIdPtr]);

					DependencyList.Add(MaterialIDElement->GetNodeId(), { EDataType::Actor, ActorData.ElementId, 0xffff });
					AddToQueue(EQueueTask::NonAsyncQueue, { AssignMaterialFunc, *MaterialElementIdPtr, { EDataType::Actor, ActorData.ElementId, 0xffff } });

					TasksToComplete |= EWorkerTask::MaterialAssign;
				}
			}
			else
			{
				for (int32 Index = 0; Index < MeshActorElement->GetMaterialOverridesCount(); ++Index)
				{
					TSharedPtr<const IDatasmithMaterialIDElement> MaterialIDElement = MeshActorElement->GetMaterialOverride(Index);

					if (FSceneGraphId* MaterialElementIdPtr = AssetElementMapping.Find(MaterialPrefix + MaterialIDElement->GetName()))
					{
						ProcessMaterialData(AssetDataList[*MaterialElementIdPtr]);

						const FString MaterialSlotName = FString::Printf(TEXT("%d"), MaterialIDElement->GetId());

						// If staticmesh has no material assigned, material assignment will be queued later when the mesh component is created
						if (StaticMaterials.Num() > 0 && SlotMapping.Contains(MaterialSlotName))
						{
							const int32 MaterialIndex = SlotMapping[MaterialSlotName];

							DependencyList.Add(MaterialIDElement->GetNodeId(), { EDataType::Actor, ActorData.ElementId, (uint16)MaterialIndex });

							AddToQueue(EQueueTask::NonAsyncQueue, { AssignMaterialFunc, *MaterialElementIdPtr, { EDataType::Actor, ActorData.ElementId, (uint16)MaterialIndex } });
							TasksToComplete |= EWorkerTask::MaterialAssign;
						}
					}
				}
			}
		}

		ActorData.SetState(EAssetState::Processed);

		return true;
	}

	bool FSceneImporter::CreateStaticMesh(FSceneGraphId ElementId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::CreateStaticMesh);

#ifdef LIVEUPDATE_TIME_LOGGING
		Timer __Timer(GlobalStartTime, "CreateStaticMesh");
#endif

		TSharedRef< IDatasmithMeshElement > MeshElement = StaticCastSharedPtr< IDatasmithMeshElement >(Elements[ElementId]).ToSharedRef();

		FAssetData& MeshData = AssetDataList[ElementId];

		UStaticMesh* StaticMesh = MeshData.GetObject<UStaticMesh>();
		if (StaticMesh == nullptr)
		{
			ensure(false);
			return false;
		}

		FDatasmithMeshElementPayload MeshPayload;
		{
			if (!Translator->LoadStaticMesh(MeshElement, MeshPayload))
			{
				// If mesh cannot be loaded, add scene's resource path if valid and retry
				bool bSecondTrySucceeded = false;

				if (FPaths::DirectoryExists(SceneElement->GetResourcePath()) && FPaths::IsRelative(MeshElement->GetFile()))
				{
					MeshElement->SetFile( *FPaths::Combine(SceneElement->GetResourcePath(), MeshElement->GetFile()) );
					bSecondTrySucceeded = Translator->LoadStaticMesh(MeshElement, MeshPayload);
				}

				if (!bSecondTrySucceeded)
				{
					// #ueent_datasmithruntime: TODO : Update FAssetFactory
					ActionCounter.Add(MeshData.Referencers.Num());
					FAssetRegistry::UnregisteredAssetsData(StaticMesh, SceneKey, [](FAssetData& AssetData) -> void
						{
							AssetData.AddState(EAssetState::Completed);
							AssetData.Object.Reset();
						});

					UE_LOGF(LogDatasmithRuntime, Warning, "CreateStaticMesh: Loading file %ls failed. Mesh element %ls has not been imported", MeshElement->GetFile(), MeshElement->GetLabel());

					return true;
				}
			}
		}

		TArray< FMeshDescription >& MeshDescriptions = MeshPayload.LodMeshes;

		// Empty mesh?
		if (MeshDescriptions.Num() == 0)
		{
			ActionCounter.Add(MeshData.Referencers.Num());
			FAssetRegistry::UnregisteredAssetsData(StaticMesh, SceneKey, [](FAssetData& AssetData) -> void
				{
					AssetData.AddState(EAssetState::Completed);
					AssetData.Object.Reset();
				});

			UE_LOGF(LogDatasmithRuntime, Warning, "CreateStaticMesh: %ls does not have a mesh description", MeshElement->GetLabel());

			return true;
		}

		// 4. Collisions
		StaticMesh->GetBodySetup()->AggGeom.EmptyElements();
		if (ImportOptions.BuildCollisions != ECollisionEnabled::NoCollision && ImportOptions.CollisionType != ECollisionTraceFlag::CTF_UseComplexAsSimple)
		{
			ProcessCollision(StaticMesh, MeshPayload);
		}

		// Extracted from FDatasmithStaticMeshImporter::SetupStaticMesh
#if WITH_EDITOR
		StaticMesh->SetNumSourceModels(MeshDescriptions.Num());
#endif

		FillStaticMeshMaterials(MeshData, MeshDescriptions);

		for (int32 LodIndex = 0; LodIndex < MeshDescriptions.Num(); ++LodIndex)
		{
			FMeshDescription& MeshDescription = MeshDescriptions[LodIndex];

			// We should always have some UV data in channel 0 because it is used in the mesh tangent calculation during the build.
			if (!DatasmithMeshHelper::HasUVData(MeshDescription, 0))
			{
				DatasmithMeshHelper::CreateDefaultUVs(MeshDescription);
			}

			// We should always have valid normals, tangents and binormals
			bool bHasInvalidNormals;
			bool bHasInvalidTangents;
			FStaticMeshOperations::HasInvalidVertexInstanceNormalsOrTangents(MeshDescription, bHasInvalidNormals, bHasInvalidTangents);

			// If normals are invalid, compute normals and tangents at polygon level then vertex level
			if (bHasInvalidNormals)
				{
				FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription, THRESH_POINTS_ARE_SAME);

				const EComputeNTBsFlags ComputeFlags = EComputeNTBsFlags::Normals | EComputeNTBsFlags::Tangents | EComputeNTBsFlags::UseMikkTSpace;
				FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, ComputeFlags);
					}
			else if (bHasInvalidTangents)
			{
				FStaticMeshOperations::ComputeMikktTangents(MeshDescription, true);
			}
		}

#if WITH_EDITOR
		// Force the generation of UVs data with full precision in the vertex buffer
		StaticMesh->GetSourceModel(0).BuildSettings.bUseFullPrecisionUVs = true;
#endif

		TArray<const FMeshDescription*> MeshDescriptionPointers;
		for (FMeshDescription& MeshDescription : MeshDescriptions)
		{
			MeshDescriptionPointers.Add(&MeshDescription);
		}

		{
			UStaticMesh::FBuildMeshDescriptionsParams Params;
			Params.bUseHashAsGuid = true;
			// Do not mark the package dirty since MarkPackageDirty is not thread safe
			Params.bMarkPackageDirty = false;
			Params.bBuildSimpleCollision = false;
			// Do not commit since we only need the render data and commit is slow
			Params.bCommitMeshDescription = false;
			Params.bFastBuild = true;
#if !WITH_EDITOR
			// Force build process to keep index buffer for complex collision when in game
			Params.bAllowCpuAccess = ImportOptions.BuildCollisions != ECollisionEnabled::NoCollision && (ImportOptions.CollisionType == ECollisionTraceFlag::CTF_UseComplexAsSimple || ImportOptions.CollisionType == ECollisionTraceFlag::CTF_UseSimpleAndComplex);
#endif

			StaticMesh->BuildFromMeshDescriptions(MeshDescriptionPointers, Params);
		}

		if (ImportOptions.BuildCollisions != ECollisionEnabled::NoCollision)
		{
			DatasmithRuntime::BuildCollision(StaticMesh->GetBodySetup(), ImportOptions.CollisionType, StaticMesh->GetRenderData()->LODResources[0]);
		}

		// Free up memory
		MeshDescriptions.Empty();

#if WITH_EDITORONLY_DATA
		StaticMesh->ClearMeshDescriptions();
#endif

		check(StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->IsInitialized());

		MeshData.ClearState(EAssetState::Building);
		FAssetRegistry::SetObjectCompletion(StaticMesh, true);

		return true;
	}

	EActionResult::Type FSceneImporter::CreateMeshComponent(FSceneGraphId ActorId, UStaticMesh* StaticMesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::CreateMeshComponent);

		if (StaticMesh == nullptr)
		{
			ActionCounter.Increment();
			return EActionResult::Succeeded;
		}

		FActorData& ActorData = ActorDataList[ActorId];

		// Component has been removed, no action needed
		if (ActorData.ElementId == INDEX_NONE)
		{
			return EActionResult::Succeeded;
		}

		IDatasmithMeshActorElement* MeshActorElement = static_cast<IDatasmithMeshActorElement*>(Elements[ActorId].Get());
		UStaticMeshComponent* MeshComponent = ActorData.GetObject<UStaticMeshComponent>();

		if (MeshComponent == nullptr)
		{
			if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None && !MeshActorElement->IsAComponent())
			{
				AStaticMeshActor* Actor = Cast<AStaticMeshActor>(RootComponent->GetOwner()->GetWorld()->SpawnActor(AStaticMeshActor::StaticClass(), nullptr, nullptr));
				check(Actor != nullptr);

				RenameObject(Actor, MeshActorElement->GetName());
#if WITH_EDITOR
				Actor->SetActorLabel( MeshActorElement->GetLabel() );
#endif

				Actor->Tags.Empty(MeshActorElement->GetTagsCount());
				for (int32 Index = 0; Index < MeshActorElement->GetTagsCount(); ++Index)
				{
					Actor->Tags.Add(MeshActorElement->GetTag(Index));
				}

				MeshComponent = Actor->GetStaticMeshComponent();
			}
			else
			{
				FName ComponentName = NAME_None;
				if (ImportOptions.BuildHierarchy != EBuildHierarchyMethod::None)
				{
					ComponentName = MakeUniqueObjectName(RootComponent->GetOwner(), UStaticMeshComponent::StaticClass(), MeshActorElement->GetLabel());
				}
				MeshComponent = NewObject< UStaticMeshComponent >(RootComponent->GetOwner(), ComponentName);
			}

			MeshComponent->ComponentTags.Add(RuntimeTag);

			MeshComponent->bAlwaysCreatePhysicsState = ImportOptions.BuildCollisions != ECollisionEnabled::NoCollision;
			MeshComponent->BodyInstance.SetCollisionEnabled(ImportOptions.BuildCollisions);

			if (MeshComponent->bAlwaysCreatePhysicsState)
			{
				MeshComponent->BodyInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
			}
			else
			{
				MeshComponent->BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			}

			ActorData.Object = TWeakObjectPtr<UObject>(MeshComponent);
		}

		FinalizeComponent(ActorData);

		MeshComponent->SetStaticMesh(StaticMesh);

		if (StaticMesh != nullptr)
		{
#ifdef ASSET_DEBUG
			StaticMesh->ClearFlags(RF_Public);
#endif

			// Allocate memory or not for override materials
			TArray< TObjectPtr<UMaterialInterface> >& OverrideMaterials = MeshComponent->OverrideMaterials;

			// There are override materials, make sure the slots are allocated
			if (MeshActorElement->GetMaterialOverridesCount() > 0)
			{
				// Update override materials if mesh element has less materials assigned than static mesh
				if (StaticMesh->GetStaticMaterials().Num() > OverrideMaterials.Num())
				{
					FActionTaskFunction AssignMaterialFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
					{
						return this->AssignMaterial(Referencer, Cast<UMaterialInstanceDynamic>(Object));
					};

					TArray< FStaticMaterial >& StaticMaterials = StaticMesh->GetStaticMaterials();

					if (MeshActorElement->GetMaterialOverride(0)->GetId() < 0)
					{
						TSharedPtr<const IDatasmithMaterialIDElement> MaterialIDElement = MeshActorElement->GetMaterialOverride(0);

						if (FSceneGraphId* MaterialElementIdPtr = AssetElementMapping.Find(MaterialPrefix + MaterialIDElement->GetName()))
						{
							DependencyList.Add(MaterialIDElement->GetNodeId(), { EDataType::Actor, ActorData.ElementId, 0xffff });
							AddToQueue(EQueueTask::NonAsyncQueue, { AssignMaterialFunc, *MaterialElementIdPtr, { EDataType::Actor, ActorData.ElementId, 0xffff } });

							TasksToComplete |= EWorkerTask::MaterialAssign;
						}
					}
					else
					{
						const int32 StaticMaterialCount = StaticMaterials.Num();

						TMap<FString, int32> SlotMapping;
						SlotMapping.Reserve(StaticMaterialCount);

						for (int32 Index = 0; Index < StaticMaterialCount; ++Index)
						{
							const FStaticMaterial& StaticMaterial = StaticMaterials[Index];

							if (StaticMaterial.MaterialSlotName != NAME_None)
							{
								SlotMapping.Add(StaticMaterial.MaterialSlotName.ToString(), Index);
							}
						}

						for (int32 Index = 0; Index < MeshActorElement->GetMaterialOverridesCount(); ++Index)
						{
							TSharedPtr<const IDatasmithMaterialIDElement> MaterialIDElement = MeshActorElement->GetMaterialOverride(Index);

							if (FSceneGraphId* MaterialElementIdPtr = AssetElementMapping.Find(MaterialPrefix + MaterialIDElement->GetName()))
							{
								const FString MaterialSlotName = FString::Printf(TEXT("%d"), MaterialIDElement->GetId());
								int32 MaterialIndex = INDEX_NONE;

								if (SlotMapping.Contains(MaterialSlotName))
								{
									MaterialIndex = SlotMapping[MaterialSlotName];
								}
								else if (MaterialIDElement->GetId() < StaticMaterialCount)
								{
									MaterialIndex = MaterialIDElement->GetId();
								}

								if (MaterialIndex != INDEX_NONE)
								{
									DependencyList.Add(MaterialIDElement->GetNodeId(), { EDataType::Actor, ActorData.ElementId, (uint16)Index });

									AddToQueue(EQueueTask::NonAsyncQueue, { AssignMaterialFunc, *MaterialElementIdPtr, { EDataType::Actor, ActorData.ElementId, (uint16)MaterialIndex } });
									TasksToComplete |= EWorkerTask::MaterialAssign;
								}
							}
						}
					}
				}

				OverrideMaterials.SetNum(StaticMesh->GetStaticMaterials().Num());
				for (int32 Index = 0; Index < OverrideMaterials.Num(); ++Index)
				{
					OverrideMaterials[Index] = nullptr;
				}

			}
			// No override material, discard the array if necessary
			else if (OverrideMaterials.Num() > 0)
			{
				OverrideMaterials.Empty();
			}
		}

		MeshComponent->MarkRenderStateDirty();

		ActorData.AddState(EAssetState::Completed);

		// Update counters
		ActionCounter.Increment();

		return EActionResult::Succeeded;
	}

	EActionResult::Type FSceneImporter::AssignMaterial(const FReferencer& Referencer, UMaterialInstanceDynamic* Material)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::AssignMaterial);

		if (Material == nullptr)
		{
			// #ue_dsruntime: Log message material not assigned
			ActionCounter.Increment();
			return EActionResult::Failed;
		}

		if (Referencer.Type == (uint8)EDataType::Mesh)
		{
			FAssetData& MeshData = AssetDataList[Referencer.GetId()];

			if (!MeshData.HasState(EAssetState::Completed))
			{
				return EActionResult::Retry;
			}

			// Static mesh can be null if creation failed
			if (UStaticMesh* StaticMesh = MeshData.GetObject<UStaticMesh>())
			{
				TArray< FStaticMaterial >& StaticMaterials = StaticMesh->GetStaticMaterials();

				if (Referencer.Slot == 0xffff)
				{
					for (FStaticMaterial& StaticMaterial : StaticMaterials)
					{
						StaticMaterial.MaterialInterface = Material;
					}
				}
				else if (!StaticMaterials.IsValidIndex(Referencer.Slot))
				{
					ensure(false);
					ActionCounter.Increment();
					return EActionResult::Failed;
				}
				else
				{
					StaticMaterials[Referencer.Slot].MaterialInterface = Material;
				}

#ifdef ASSET_DEBUG
				Material->ClearFlags(RF_Public);
#endif
				// Mark dependent mesh components' render state as dirty
				for (FReferencer& ActorReferencer : MeshData.Referencers)
				{
					FActorData& ActorData = ActorDataList[ActorReferencer.GetId()];

					if (UActorComponent* ActorComponent = ActorData.GetObject<UActorComponent>())
					{
						ActorComponent->MarkRenderStateDirty();
					}
				}
			}
		}
		else if (Referencer.Type == (uint8)EDataType::Actor)
		{
			FActorData& ActorData = ActorDataList[Referencer.GetId()];

			if (!ActorData.HasState(EAssetState::Completed))
			{
				return EActionResult::Retry;
			}

			// Static mesh can be null if creation failed
			if (UStaticMeshComponent* MeshComponent = ActorData.GetObject<UStaticMeshComponent>())
			{
				if (Referencer.Slot == 0xffff)
				{
					for (int32 Index = 0; Index < MeshComponent->GetNumMaterials(); ++Index)
					{
						MeshComponent->SetMaterial(Index, Material);
					}
				}
				else if ((int32)Referencer.Slot >= MeshComponent->GetNumMaterials())
				{
					ensure(false);
					ActionCounter.Increment();
					return EActionResult::Failed;
				}
				else
				{
					MeshComponent->SetMaterial(Referencer.Slot, Material);
				}

				// Force rebuilding of render data for mesh component
				MeshComponent->MarkRenderStateDirty();
#ifdef ASSET_DEBUG
				Material->ClearFlags(RF_Public);
#endif
			}
			else
			{
				ensure(false);
				ActionCounter.Increment();
				return EActionResult::Failed;
			}
		}
		else
		{
			ensure(false);
			ActionCounter.Increment();
			return EActionResult::Failed;
		}

		ActionCounter.Increment();

		return EActionResult::Succeeded;
	}

	void FSceneImporter::UpdateStaticMeshMaterials(FAssetData& MeshData)
	{
		TSharedPtr< IDatasmithMeshElement > MeshElement = StaticCastSharedPtr< IDatasmithMeshElement >(Elements[MeshData.ElementId]);
		UStaticMesh* StaticMesh = MeshData.GetObject<UStaticMesh>();

		if (!MeshElement.IsValid() || StaticMesh == nullptr)
		{
			return;
		}

		TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
		const int32 MaterialSlotCount = StaticMaterials.Num();

		TMap<FString, int32> SlotMapping;
		SlotMapping.Reserve(MaterialSlotCount);

		for(int32 Index = 0; Index < MaterialSlotCount; ++Index)
		{
			const FStaticMaterial& StaticMaterial = StaticMaterials[Index];
			SlotMapping.Add(StaticMaterial.MaterialSlotName.ToString(), Index);
		}

		FActionTaskFunction AssignMaterialFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
		{
			return this->AssignMaterial(Referencer, Cast<UMaterialInstanceDynamic>(Object));
		};

		TFunction<bool(const IDatasmithMaterialIDElement*,int32)> UpdateMaterialSlot;
		UpdateMaterialSlot = [&](const IDatasmithMaterialIDElement* MaterialIDElement, int32 SlotIndex) -> bool
		{
			const FString MaterialPathName(MaterialIDElement->GetName());

			UMaterialInterface* PreviousMaterialInterface = StaticMaterials[SlotIndex].MaterialInterface;

			if (!MaterialPathName.StartsWith(TEXT("/")))
			{
				StaticMaterials[SlotIndex].MaterialInterface = nullptr;

				if (FSceneGraphId* MaterialElementIdPtr = AssetElementMapping.Find(MaterialPrefix + MaterialPathName))
				{
					DependencyList.Add(MaterialIDElement->GetNodeId(), { EDataType::Mesh, MeshData.ElementId, (uint16)SlotIndex });

					AddToQueue(EQueueTask::NonAsyncQueue, { AssignMaterialFunc, *MaterialElementIdPtr, { EDataType::Mesh, MeshData.ElementId, (uint16)SlotIndex } });
					TasksToComplete |= EWorkerTask::MaterialAssign;
				}
				else
				{
					DependencyList.Remove(MaterialIDElement->GetNodeId());
				}
			}
			else
			{
				StaticMaterials[SlotIndex].MaterialInterface = Cast<UMaterialInterface>(FSoftObjectPath(MaterialPathName).TryLoad());
			}

			return PreviousMaterialInterface != StaticMaterials[SlotIndex].MaterialInterface;
		};

		bool bUpdateReferencers = false;

		// Check to see if there is material to apply on all slots
		int32 OverrideIndex = INDEX_NONE;
		for (int32 Index = 0; Index < MeshElement->GetMaterialSlotCount(); Index++)
		{
			if (MeshElement->GetMaterialSlotAt(Index).IsValid() && MeshElement->GetMaterialSlotAt(Index)->GetId() < 0)
			{
				OverrideIndex = Index;
				break;
			}
		}

		if (OverrideIndex != INDEX_NONE)
		{
			const IDatasmithMaterialIDElement* MaterialIDElement = MeshElement->GetMaterialSlotAt(OverrideIndex).Get();

			for (int32 Index = 0; Index < MaterialSlotCount; Index++)
			{
				bUpdateReferencers |= UpdateMaterialSlot(MaterialIDElement, Index);
			}
		}

		// Apply material on specific slots
		for (int32 Index = 0; Index < MeshElement->GetMaterialSlotCount(); Index++)
		{
			if (const IDatasmithMaterialIDElement* MaterialIDElement = MeshElement->GetMaterialSlotAt(Index).Get())
			{
				if (MaterialIDElement->GetId() >= 0)
				{
					const FString MaterialSlotName = FString::Printf(TEXT("%d"), MaterialIDElement->GetId());
					int32 SlotIndex = INDEX_NONE;
					if (SlotMapping.Contains(MaterialSlotName))
					{
						SlotIndex = SlotMapping[MaterialSlotName];
					}
					else if (MaterialIDElement->GetId() < MaterialSlotCount)
					{
						SlotIndex = MaterialIDElement->GetId();
					}
					else
					{
						UE_LOGF(LogDatasmithRuntime, Warning, "CreateStaticMesh: Cannot assign material %ls to any slot", MaterialIDElement->GetName());
						continue;
					}

					bUpdateReferencers |= UpdateMaterialSlot(MaterialIDElement, SlotIndex);
				}
			}
		}

		if (bUpdateReferencers)
		{
			// Mark dependent mesh components' render state as dirty
			for (FReferencer& ActorReferencer : MeshData.Referencers)
			{
				const FActorData& ActorData = ActorDataList[ActorReferencer.GetId()];

				if (ActorData.HasState(EAssetState::Completed))
				{
					if (UActorComponent* ActorComponent = ActorData.GetObject<UActorComponent>())
					{
						ActorComponent->MarkRenderStateDirty();
					}
				}
			}
		}

	}

	void FSceneImporter::FillStaticMeshMaterials(FAssetData& MeshData, TArray<FMeshDescription>& MeshDescriptions)
	{
		TSharedRef< IDatasmithMeshElement > MeshElement = StaticCastSharedPtr< IDatasmithMeshElement >(Elements[MeshData.ElementId]).ToSharedRef();
		UStaticMesh* StaticMesh = MeshData.GetObject<UStaticMesh>();

		if (StaticMesh == nullptr)
		{
			return;
		}

		TMap<FString, int32> SlotMapping;

		// Update static mesh's static material array for LOD 0
		TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
		FMeshDescription& MeshDescription = MeshDescriptions[0];

		FStaticMeshAttributes Attributes(MeshDescription);
		TPolygonGroupAttributesConstRef<FName> MaterialSlotNameAttribute = Attributes.GetPolygonGroupMaterialSlotNames();

		const int32 MaterialSlotCount = MeshDescription.PolygonGroups().Num();

		StaticMaterials.SetNum(MaterialSlotCount);

		{
			int32 Index = 0;
			for (FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
			{
				FStaticMaterial& StaticMaterial = StaticMaterials[Index];

				StaticMaterial.MaterialSlotName = MaterialSlotNameAttribute[PolygonGroupID];
				StaticMaterial.MaterialInterface = nullptr;
				// Done to remove an assert from an 'ensure' in UStaticMesh::GetUVChannelData
				StaticMaterial.UVChannelData = FMeshUVChannelInfo(1.f);

				++Index;
			}
		}

		// Add task to update material interfaces on static materials if applicable
		if (MeshElement->GetMaterialSlotCount() > 0)
		{
			FActionTaskFunction UpdateMaterialsFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
			{
				this->UpdateStaticMeshMaterials(AssetDataList[Referencer.GetId()]);
				return EActionResult::Succeeded;
			};

			AddToQueue(EQueueTask::NonAsyncQueue, { UpdateMaterialsFunc, DirectLink::InvalidId, { EDataType::Mesh, MeshData.ElementId, 0 } });
			TasksToComplete |= EWorkerTask::MaterialAssign;
		}

		// Add slots defined in subsequent LODs but not present in LOD 0
		TSet<FName> LODSlotNames;
		for (int32 LODIndex = 1; LODIndex < MeshDescriptions.Num(); ++LODIndex)
		{
			FMeshDescription& LODMeshDescription = MeshDescriptions[LODIndex];

			FStaticMeshAttributes LODAttributes(LODMeshDescription);
			TPolygonGroupAttributesConstRef<FName> LODMaterialSlotNameAttribute = LODAttributes.GetPolygonGroupMaterialSlotNames();

			for (FPolygonGroupID PolygonGroupID : LODMeshDescription.PolygonGroups().GetElementIDs())
			{
				const FName LODSlotName = LODMaterialSlotNameAttribute[PolygonGroupID];
				if (!SlotMapping.Contains(LODSlotName.ToString()))
				{
					LODSlotNames.Add(LODSlotName);
				}
			}
		}

		if (LODSlotNames.Num() > 0)
		{
			StaticMaterials.SetNum(MaterialSlotCount + LODSlotNames.Num());
			int32 Index = MaterialSlotCount;
			for (FName& SlotName : LODSlotNames)
			{
				FStaticMaterial& StaticMaterial = StaticMaterials[Index];
				StaticMaterial.MaterialSlotName = SlotName;
				StaticMaterial.MaterialInterface = nullptr;
				// Done to remove an assert from an 'ensure' in UStaticMesh::GetUVChannelData
				StaticMaterial.UVChannelData = FMeshUVChannelInfo(1.f);

				++Index;
			}
		}
	}
}
