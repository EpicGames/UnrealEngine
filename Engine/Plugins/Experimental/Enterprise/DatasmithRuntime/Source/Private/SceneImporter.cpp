// Copyright Epic Games, Inc. All Rights Reserved.

#include "SceneImporter.h"

#include "DatasmithRuntimeUtils.h"
#include "LogCategory.h"
#include "MaterialImportUtils.h"

#include "DatasmithAssetUserData.h"
#include "DatasmithNativeTranslator.h"
#include "DatasmithMaterialElements.h"

#include "Async/Async.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"
#include "Misc/Paths.h"

namespace DatasmithRuntime
{
	extern void UpdateMaterials(TSet<FSceneGraphId>& MaterialElementSet, TMap< FSceneGraphId, FAssetData >& AssetDataList);
	extern void HideSceneComponent(USceneComponent* SceneComponent);

#ifdef LIVEUPDATE_TIME_LOGGING
	Timer::Timer(double InTimeOrigin, const char* InText)
		: TimeOrigin(InTimeOrigin)
		, StartTime(FPlatformTime::Seconds())
		, Text(InText)
	{
	}

	Timer::~Timer()
	{
		const double EndTime = FPlatformTime::Seconds();
		const double ElapsedMilliSeconds = (EndTime - StartTime) * 1000.;

		double SecondsSinceOrigin = EndTime - TimeOrigin;

		const int MinSinceOrigin = int(SecondsSinceOrigin / 60.);
		SecondsSinceOrigin -= 60.0 * (double)MinSinceOrigin;

		UE_LOGF(LogDatasmithRuntime, Log, "%ls in [%.3f ms] ( since beginning [%d min %.3f s] )", *Text, ElapsedMilliSeconds, MinSinceOrigin, SecondsSinceOrigin);
	}
#endif

	const FString TexturePrefix( TEXT( "Texture." ) );
	const FString MaterialPrefix( TEXT( "Material." ) );
	const FString MeshPrefix( TEXT( "Mesh." ) );

	FAssetData FAssetData::EmptyAsset(DirectLink::InvalidId);

	FSceneImporter::FSceneImporter(ADatasmithRuntimeActor* InDatasmithRuntimeActor)
		: RootComponent( InDatasmithRuntimeActor->GetRootComponent() )
		, TasksToComplete( EWorkerTask::NoTask )
		, OverallProgress(InDatasmithRuntimeActor->Progress)
	{
		SceneKey = GetTypeHash(FGuid::NewGuid());
		FAssetRegistry::RegisterMapping(SceneKey, &AssetDataList);

		FAssetData::EmptyAsset.SetState(EAssetState::Processed | EAssetState::Completed);
	}

	FSceneImporter::~FSceneImporter()
	{
		DeleteData();
		FAssetRegistry::UnregisterMapping(SceneKey);
	}

	TStatId FSceneImporter::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FSceneImporter, STATGROUP_Tickables);
	}

	void FSceneImporter::ParseScene( const TSharedPtr<IDatasmithActorElement>& ActorElement, FSceneGraphId ParentId, FParsingCallback Callback )
	{
		Callback( ActorElement, ParentId );

		FSceneGraphId ActorId = ActorElement->GetNodeId();

		for (int32 Index = 0; Index < ActorElement->GetChildrenCount(); ++Index)
		{
			ParseScene( ActorElement->GetChild(Index), ActorId, Callback );
		}
	}

	void FSceneImporter::StartImport(TSharedRef<IDatasmithScene> InSceneElement, const FDatasmithRuntimeImportOptions& Options)
	{
		Reset(true);

		ImportOptions = Options;

		SceneElement = InSceneElement;

		if (SceneElement.IsValid() && !Translator.IsValid())
		{
			Translator = MakeShared<FDatasmithNativeTranslator>();
		}

		TasksToComplete |= SceneElement.IsValid() ? EWorkerTask::CollectSceneData : EWorkerTask::NoTask;

#ifdef LIVEUPDATE_TIME_LOGGING
		GlobalStartTime = FPlatformTime::Seconds();
#endif
	}

	void FSceneImporter::AddAsset(TSharedPtr<IDatasmithElement>&& InElementPtr, const FString& AssetPrefix, EDataType DataType)
	{
		if (IDatasmithElement* Element = InElementPtr.Get())
		{
			const FString AssetKey = AssetPrefix + Element->GetName();
			const FSceneGraphId ElementId = Element->GetNodeId();

			AssetElementMapping.Add( AssetKey, ElementId );

			Elements.Add( ElementId, MoveTemp( InElementPtr ) );

			FAssetData AssetData(ElementId, DataType);
			AssetDataList.Emplace(ElementId, MoveTemp(AssetData));
		}
	}

	void FSceneImporter::CollectSceneData()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::CollectSceneData);

		LIVEUPDATE_LOG_TIME;

		int32 ActorElementCount = 0;

		for (int32 Index = 0; Index < SceneElement->GetActorsCount(); ++Index)
		{
			ParseScene( SceneElement->GetActor(Index), DirectLink::InvalidId,
				[&](const TSharedPtr< IDatasmithActorElement>& ActorElement, FSceneGraphId ActorId) -> void
				{
					++ActorElementCount;
				});
		}

		int32 AssetElementCount = SceneElement->GetTexturesCount() + SceneElement->GetMaterialsCount() +
			SceneElement->GetMeshesCount() + SceneElement->GetLevelSequencesCount();

		if (bIncrementalUpdate)
		{
			// Make sure to pre-allocate enough memory as pointer on values in those maps are used
			TextureDataList.Reserve( FMath::Max(TextureDataList.Num(), SceneElement->GetTexturesCount()) );
			AssetDataList.Reserve( FMath::Max(AssetDataList.Num(), AssetElementCount) );
			ActorDataList.Reserve( FMath::Max(ActorDataList.Num(), ActorElementCount) );
			Elements.Reserve( FMath::Max(Elements.Num(), AssetElementCount + ActorElementCount) );
			DependencyList.Reserve(FMath::Max(DependencyList.Num(), SceneElement->GetMeshesCount() + SceneElement->GetMetaDataCount()));

			AssetElementMapping.Reserve( FMath::Max(AssetElementMapping.Num(), AssetElementCount) );

			// Reset counters
			QueuedTaskCount = 0;

			// Parse scene to collect all actions to be taken
			for (int32 Index = 0; Index < SceneElement->GetActorsCount(); ++Index)
			{
				ParseScene(SceneElement->GetActor(Index), DirectLink::InvalidId,
					[this](const TSharedPtr<IDatasmithActorElement>& ActorElement, FSceneGraphId ParentId) -> void
					{
						this->ProcessActorElement(ActorElement, ParentId);
					}
				);
			}

			// #ue_datasmithruntime: What about lightmap weights on incremental update?
			LightmapWeights.Empty();

			bIncrementalUpdate = false;
		}
		else
		{
			// Make sure to pre-allocate enough memory as pointer on values in those maps are used
			TextureDataList.Empty( SceneElement->GetTexturesCount() );
			AssetDataList.Empty( AssetElementCount );
			ActorDataList.Empty( ActorElementCount );
			Elements.Empty( AssetElementCount + ActorElementCount );
			DependencyList.Empty(SceneElement->GetMeshesCount());

			AssetElementMapping.Empty( AssetElementCount );

			for (int32 Index = 0; Index < SceneElement->GetTexturesCount(); ++Index)
			{
				// Only add a texture if its associated resource file is available
				if (IDatasmithTextureElement* TextureElement = static_cast<IDatasmithTextureElement*>(SceneElement->GetTexture(Index).Get()))
				{
					// If resource file does not exist, add scene's resource path if valid
					if (!FPaths::FileExists(TextureElement->GetFile()) && FPaths::DirectoryExists(SceneElement->GetResourcePath()))
					{
						TextureElement->SetFile( *FPaths::Combine(SceneElement->GetResourcePath(), TextureElement->GetFile()) );
					}

					if (FPaths::FileExists(TextureElement->GetFile()))
					{
						AddAsset(SceneElement->GetTexture(Index), TexturePrefix, EDataType::Texture);
					}
					else
					{
						EDatasmithTextureFormat TextureFormat;
						uint32                  ByteCount;

						if (TextureElement->GetData(ByteCount, TextureFormat) != nullptr && ByteCount > 0)
						{
							AddAsset(SceneElement->GetTexture(Index), TexturePrefix, EDataType::Texture);
						}
					}
				}
				// #ueent_datasmithruntime: Inform user resource file does not exist
			}

			for (int32 Index = 0; Index < SceneElement->GetMaterialsCount(); ++Index)
			{
				AddAsset(SceneElement->GetMaterial(Index), MaterialPrefix, EDataType::Material);
			}

			for (int32 Index = 0; Index < SceneElement->GetMeshesCount(); ++Index)
			{
				// Only add a mesh if its associated resource is available
				if (IDatasmithMeshElement* MeshElement = static_cast<IDatasmithMeshElement*>(SceneElement->GetMesh(Index).Get()))
				{
					AddAsset(SceneElement->GetMesh(Index), MeshPrefix, EDataType::Mesh);
				}
				// #ueent_datasmithruntime: Inform user resource file does not exist
			}

			// Collect set of materials and meshes used in scene
			// Collect set of textures used in scene
			TextureElementSet.Empty(SceneElement->GetTexturesCount());
			MeshElementSet.Empty(SceneElement->GetMeshesCount());
			MaterialElementSet.Empty(SceneElement->GetMaterialsCount());

			for (int32 Index = 0; Index < SceneElement->GetActorsCount(); ++Index)
			{
				ParseScene(SceneElement->GetActor(Index), DirectLink::InvalidId,
					[this](const TSharedPtr<IDatasmithActorElement>& ActorElement, FSceneGraphId ParentId) -> void
					{
						this->ProcessActorElement(ActorElement, ParentId);
					}
				);
			}

			if (ImportOptions.bImportMetaData)
			{
				// Start collection of metadata
				MetadataCollect = Async(
#if WITH_EDITOR
					EAsyncExecution::LargeThreadPool,
#else
					EAsyncExecution::ThreadPool,
#endif
					[this]() -> void
					{
						for (int32 Index = 0; Index < this->SceneElement->GetMetaDataCount(); ++Index)
						{
							this->ProcessMetdata(this->SceneElement->GetMetaData(Index));
						}
					}
				);
			}
		}

		TasksToComplete |= EWorkerTask::SetupTasks;
	}

	void FSceneImporter::SetupTasks()
	{
		LIVEUPDATE_LOG_TIME;

		// Compute parameters for update on progress
		int32 ActionsCount = QueuedTaskCount;

		ActionsCount += MaterialElementSet.Num();

		if (TextureElementSet.Num() > 0)
		{
			ImageReaderInitialize();

			TasksToComplete |= EWorkerTask::TextureLoad;
		}

		// Add image load + texture creation + texture assignments
		for (FSceneGraphId ElementId : TextureElementSet)
		{
			const FAssetData& AssetData = AssetDataList[ElementId];
			ActionsCount += AssetData.Referencers.Num() + 2;
		}

		OverallProgress = 0.05f;
		double MaxActions = FMath::FloorToDouble( (double)ActionsCount / 0.95 );
		ActionCounter.Set((int32)FMath::CeilToDouble( MaxActions * 0.05 ));
		ProgressStep = 1. / MaxActions;

		OnGoingTasks.Reserve(TextureElementSet.Num() + MeshElementSet.Num());
	}

	void FSceneImporter::Tick(float DeltaSeconds)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::Tick);

		if (TasksToComplete == EWorkerTask::NoTask)
		{
			return;
		}

		// Full reset of the world. Resume tasks on next tick
		if (EnumHasAnyFlags( TasksToComplete, EWorkerTask::ResetScene))
		{
			// Wait for ongoing tasks to be completed
			for (TFuture<bool>& OnGoingTask : OnGoingTasks)
			{
				OnGoingTask.Wait();
			}

			OnGoingTasks.Empty();

			bool bGarbageCollect = DeleteData();

			Elements.Empty();
			AssetElementMapping.Empty();

			AssetDataList.Empty();
			TextureDataList.Empty();
			ActorDataList.Empty();
			DependencyList.Empty();

			bGarbageCollect |= FAssetRegistry::CleanUp();

			TasksToComplete &= ~EWorkerTask::ResetScene;

			// If there is no more tasks to complete, delete assets which are not used
			if (bGarbageCollect)
			{
				if (!IsGarbageCollecting())
				{
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				}
				else
				{
					// Post-pone garbage collection for next frame
					TasksToComplete = EWorkerTask::GarbageCollect;
				}
			}

			return;
		}

		struct FLocalUpdate
		{
			FLocalUpdate(float& InProgress, FThreadSafeCounter& InCounter, double InStep)
				: Progress(InProgress)
				, Counter(InCounter)
				, Step(InStep)
			{
			}

			~FLocalUpdate()
			{
				Progress = (float)((double)Counter.GetValue() * Step);
			}

			float& Progress;
			FThreadSafeCounter& Counter;
			double Step;
		};

		FLocalUpdate LocalUpdate(OverallProgress, ActionCounter, ProgressStep);

		// Execute work by chunk of 10 milliseconds timespan
		double EndTime = FPlatformTime::Seconds() + 0.02;

		if (EnumHasAnyFlags( TasksToComplete, EWorkerTask::GarbageCollect))
		{
			// Do not take any risk, wait for next frame to continue the process
			if (IsGarbageCollecting())
			{
				return;
			}

#ifdef LIVEUPDATE_TIME_LOGGING
			Timer(GlobalStartTime, "GarbageCollect");
#endif

			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			TasksToComplete &= ~EWorkerTask::GarbageCollect;
		}

		bool bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::DeleteComponent))
		{
#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "DeleteComponent");
#endif

			FActionTask ActionTask;
			while (FPlatformTime::Seconds() < EndTime)
			{
				if (!ActionQueues[EQueueTask::DeleteCompQueue].Dequeue(ActionTask))
				{
					TasksToComplete &= ~EWorkerTask::DeleteComponent;
					break;
				}

				ActionTask.Execute(FAssetData::EmptyAsset);
			}
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		// Do not continue if there are still components to garbage collect
		// Force a garbage collection if we are done with the components
		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::GarbageCollect))
		{
			// Terminate all rendering commands before deleting any component
			FlushRenderingCommands();

			if (IsGarbageCollecting())
			{
				return;
			}

#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "GarbageCollect");
#endif

			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			TasksToComplete &= ~EWorkerTask::GarbageCollect;
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::DeleteAsset))
		{
#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "DeleteAsset");
#endif

			FActionTask ActionTask;
			while (FPlatformTime::Seconds() < EndTime)
			{
				if (!ActionQueues[EQueueTask::DeleteAssetQueue].Dequeue(ActionTask))
				{
					TasksToComplete &= ~EWorkerTask::DeleteAsset;
					break;
				}

				ActionTask.Execute(FAssetData::EmptyAsset);
			}
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::CollectSceneData))
		{
			CollectSceneData();
			TasksToComplete &= ~EWorkerTask::CollectSceneData;
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::UpdateElement))
		{
#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "UpdateElement");
#endif

			ProcessQueue(EQueueTask::UpdateQueue, EndTime, EWorkerTask::UpdateElement, EWorkerTask::SetupTasks);
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags( TasksToComplete, EWorkerTask::SetupTasks))
		{
			SetupTasks();
			TasksToComplete &= ~EWorkerTask::SetupTasks;
		}

		// Do not proceed further if metadata collection is not complete
		if (MetadataCollect.IsValid() && !MetadataCollect.IsReady())
		{
			return;
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::MeshCreate))
		{
#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "MeshCreate");
#endif

			ProcessQueue(EQueueTask::MeshQueue, EndTime, EWorkerTask::MeshCreate);
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::MaterialCreate))
		{
#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "MaterialCreate");
#endif

			FActionTask ActionTask;
			while (FPlatformTime::Seconds() < EndTime)
			{
				if (!ActionQueues[EQueueTask::MaterialQueue].Dequeue(ActionTask))
				{
					TasksToComplete &= ~EWorkerTask::MaterialCreate;
					if (!EnumHasAnyFlags(TasksToComplete, EWorkerTask::TextureAssign))
					{
						UpdateMaterials(MaterialElementSet, AssetDataList);
					}
					break;
				}

				ensure(DirectLink::InvalidId == ActionTask.GetElementId());
				ActionTask.Execute(FAssetData::EmptyAsset);
			}
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::TextureLoad))
		{
#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "TextureLoad");
#endif

			ProcessQueue(EQueueTask::TextureQueue, EndTime, EWorkerTask::TextureLoad);
		}

		bContinue = FPlatformTime::Seconds() < EndTime;

		if (bContinue && EnumHasAnyFlags(TasksToComplete, EWorkerTask::NonAsyncTasks))
		{
#ifdef LIVEUPDATE_TIME_LOGGING
			Timer __Timer(GlobalStartTime, "GameThreadTasks");
#endif

			FActionTask ActionTask;
			while (FPlatformTime::Seconds() < EndTime)
			{
				if (!ActionQueues[EQueueTask::NonAsyncQueue].Dequeue(ActionTask))
				{
					if (EnumHasAnyFlags(TasksToComplete, EWorkerTask::TextureAssign))
					{
						UpdateMaterials(MaterialElementSet, AssetDataList);
					}
					TasksToComplete &= ~EWorkerTask::NonAsyncTasks;
					break;
				}

				const FSceneGraphId ElementId = ActionTask.GetElementId();
				FBaseData& ElementData = DirectLink::InvalidId == ElementId ? FAssetData::EmptyAsset : (AssetDataList.Contains(ElementId) ? (FBaseData&)AssetDataList[ElementId] : (FBaseData&)ActorDataList[ElementId]);
				if (ActionTask.Execute(ElementData) == EActionResult::Retry)
				{
					ActionQueues[EQueueTask::NonAsyncQueue].Enqueue(MoveTemp(ActionTask));
					continue;
				}
			}
		}

		if (TasksToComplete == EWorkerTask::NoTask && SceneElement.IsValid())
		{
			// Terminate all rendering commands before deleting any asset
			FlushRenderingCommands();

			// Delete assets which has not been reused on the last processing
			if (FAssetRegistry::CleanUp())
			{
				if (!IsGarbageCollecting())
				{
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				}
				else
				{
					// Garbage collection has not been performed. Do it on next frame
					TasksToComplete = EWorkerTask::GarbageCollect;
					return;
				}
			}

			TRACE_BOOKMARK(TEXT("Load complete - %s"), SceneElement->GetName());

			OnGoingTasks.Empty();

			LastSceneGuid = SceneElement->GetSharedState()->GetGuid();

			// Free up the translator since it is not needed anymore
			Translator.Reset();
			Cast<ADatasmithRuntimeActor>(RootComponent->GetOwner())->OnImportEnd();
#ifdef LIVEUPDATE_TIME_LOGGING
			double ElapsedSeconds = FPlatformTime::Seconds() - GlobalStartTime;

			int ElapsedMin = int(ElapsedSeconds / 60.0);
			ElapsedSeconds -= 60.0 * (double)ElapsedMin;

			UE_LOGF(LogDatasmithRuntime, Log, "Total load time is [%d min %.3f s]", ElapsedMin, ElapsedSeconds);
#endif
		}
	}

	void FSceneImporter::Reset(bool bIsNewScene)
	{
		bIncrementalUpdate = false;

		// Hide all imported scene components if a new scene is going to be imported.
		if (bIsNewScene)
		{
			for (TPair< FSceneGraphId, FActorData >& Pair : ActorDataList)
			{
				if (USceneComponent* SceneComponent = Pair.Value.GetObject<USceneComponent>())
				{
					HideSceneComponent(SceneComponent);
				}
			}
		}

		// Clear all cached data if it is a new scene
		SceneElement.Reset();
		LastSceneGuid = FGuid();
		MetadataCollect = TFuture<void>();

		TasksToComplete = EWorkerTask::ResetScene;

		// Empty tasks queues
		for (TQueue< FActionTask, EQueueMode::Mpsc >& Queue : ActionQueues)
		{
			Queue.Empty();
		}

		// Reset counters
		QueuedTaskCount = 0;

		// Empty tracking arrays and sets
		MeshElementSet.Empty();
		TextureElementSet.Empty();
		MaterialElementSet.Empty();
		// #ue_datasmithruntime: What about lightmap weights on incremental update?
		LightmapWeights.Empty();
	}

	bool FSceneImporter::IncrementalUpdate(TSharedRef< IDatasmithScene > InSceneElement, FUpdateContext& UpdateContext)
	{
#ifdef LIVEUPDATE_TIME_LOGGING
		GlobalStartTime = FPlatformTime::Seconds();
#endif
		UE_LOGF(LogDatasmithRuntime, Log, "Incremental update...");

		SceneElement = InSceneElement;
		ensure(SceneElement.IsValid());

		Translator = MakeShared<FDatasmithNativeTranslator>();

		PrepareIncrementalUpdate(UpdateContext);

		IncrementalAdditions(UpdateContext.Additions, UpdateContext.Updates);

		IncrementalModifications(UpdateContext.Updates);

		IncrementalDeletions(UpdateContext.Deletions);

		bIncrementalUpdate = true;

		TasksToComplete |= EWorkerTask::CollectSceneData;

		return true;
	}

	void FSceneImporter::IncrementalModifications(TArray<TSharedPtr<IDatasmithElement>>& Modifications)
	{
		if (Modifications.Num() == 0)
		{
			return;
		}

		for (TSharedPtr<IDatasmithElement>& ElementPtr : Modifications)
		{
			if (Elements.Contains(ElementPtr->GetNodeId()))
			{
				FSceneGraphId ElementId = ElementPtr->GetNodeId();

				if (AssetDataList.Contains(ElementId))
				{
					const EDataType DataType = AssetDataList[ElementId].Type;
					const FString& Prefix = DataType == EDataType::Texture ? TexturePrefix : (DataType == EDataType::Material ? MaterialPrefix : MeshPrefix);

					UE_LOGF(LogDatasmithRuntime, Log, "IncrementalModifications: %ls %ls (%d)", *Prefix, ElementPtr->GetName(), ElementPtr->GetNodeId());

					const FString PrefixedName = Prefix + ElementPtr->GetName();

					if (!AssetElementMapping.Contains(PrefixedName))
					{
						for (TPair<FString, FSceneGraphId>& Entry : AssetElementMapping)
						{
							if (Entry.Value == ElementId)
							{
								const FString OldKey = Entry.Key;
								AssetElementMapping.Remove(OldKey);
								break;
							}
						}

						AssetElementMapping.Add(PrefixedName, ElementId);
					}

					FActionTaskFunction TaskFunc;

					if (ElementPtr->IsA(EDatasmithElementType::BaseMaterial))
					{
						TaskFunc = [this, ElementId](UObject*, const FReferencer&) -> EActionResult::Type
						{
							FAssetData& MaterialData = this->AssetDataList[ElementId];

							MaterialData.SetState(EAssetState::Unknown);

							this->ProcessMaterialData(MaterialData);

							ActionCounter.Increment();

							return EActionResult::Succeeded;
						};
					}
					else if (ElementPtr->IsA(EDatasmithElementType::StaticMesh))
					{
						TaskFunc = [this, ElementId](UObject*, const FReferencer&) -> EActionResult::Type
						{
							FAssetData& MeshData = this->AssetDataList[ElementId];

							MeshData.SetState(EAssetState::Unknown);

							ActionCounter.Increment();

							this->ProcessMeshData(MeshData);

							return EActionResult::Succeeded;
						};
					}
					else if (ElementPtr->IsA(EDatasmithElementType::Texture))
					{
						ensure(TextureDataList.Contains(ElementId));

						TaskFunc = [this, ElementId](UObject*, const FReferencer&) -> EActionResult::Type
						{
							FAssetData& TextureData = this->AssetDataList[ElementId];

							TextureData.SetState(EAssetState::Unknown);

							this->ProcessTextureData(ElementId);

							ActionCounter.Increment();

							return EActionResult::Succeeded;
						};
					}

					AddToQueue(EQueueTask::UpdateQueue, { MoveTemp(TaskFunc), FReferencer() } );

					TasksToComplete |= EWorkerTask::SetupTasks;
				}
				else if (ActorDataList.Contains(ElementId))
				{
					FActorData& ActorData = ActorDataList[ElementId];

					UE_LOGF(LogDatasmithRuntime, Log, "IncrementalModifications: Actor %ls (%d)", ElementPtr->GetName(), ElementPtr->GetNodeId());

					ActorData.SetState(EAssetState::Unknown);
				}
			}
			else if (DependencyList.Contains(ElementPtr->GetNodeId()))
			{
				ProcessDependency(ElementPtr);
			}
		}
	}

	void FSceneImporter::IncrementalDeletions(TArray<DirectLink::FSceneGraphId>& Deletions)
	{
		if (Deletions.Num() == 0)
		{
			return;
		}

		FActionTaskFunction TaskFunc = [this](UObject*, const FReferencer& Referencer) -> EActionResult::Type
		{
			EActionResult::Type Result = this->DeleteElement(Referencer.GetId());

			if(Result == EActionResult::Succeeded)
			{
				this->TasksToComplete |= EWorkerTask::GarbageCollect;
			}

			return Result;
		};

		bool bFlushRenderingCommands = false;

		for (DirectLink::FSceneGraphId& ElementId : Deletions)
		{
			if (Elements.Contains(ElementId))
			{
				if (AssetDataList.Contains(ElementId))
				{
					if (!AssetDataList[ElementId].HasState(EAssetState::PendingDelete))
					{
						continue;
					}

					const EDataType DataType = AssetDataList[ElementId].Type;
					const FString& Prefix = DataType == EDataType::Texture ? TexturePrefix : (DataType == EDataType::Material ? MaterialPrefix : MeshPrefix);

					UE_LOGF(LogDatasmithRuntime, Log, "IncrementalDeletions: %ls %d", *Prefix, ElementId);

					AddToQueue(EQueueTask::DeleteAssetQueue, { TaskFunc, FReferencer(ElementId) } );
					TasksToComplete |= EWorkerTask::DeleteAsset;
				}
				else if (ActorDataList.Contains(ElementId))
				{
					UE_LOGF(LogDatasmithRuntime, Log, "IncrementalDeletions: actor %ls (%d)", Elements[ElementId]->GetLabel(), ElementId);

					HideSceneComponent(ActorDataList[ElementId].GetObject<USceneComponent>());

					AddToQueue(EQueueTask::DeleteCompQueue, { TaskFunc, FReferencer(ElementId) } );
					TasksToComplete |= EWorkerTask::DeleteComponent;

					bFlushRenderingCommands = true;
				}
				// Remove metadata from list of tracked elements
				else if (Elements[ElementId]->IsA(EDatasmithElementType::MetaData))
				{
					AddToQueue(EQueueTask::DeleteCompQueue, { TaskFunc, FReferencer(ElementId) } );
					TasksToComplete |= EWorkerTask::DeleteComponent;
				}
				else
				{
					UE_LOGF(LogDatasmithRuntime, Error, "Element %d (%ls) was not found", ElementId, Elements[ElementId]->GetName());
					ensure(false);
				}
			}
		}

		if (bFlushRenderingCommands)
		{
			FlushRenderingCommands();
		}
	}

	void FSceneImporter::ProcessDependency(const TSharedPtr<IDatasmithElement>& Element)
	{
		FSceneGraphId ElementId = Element->GetNodeId();
		FReferencer Referencer = DependencyList[ElementId];

		if (Element->IsA(EDatasmithElementType::MaterialId))
		{
			const IDatasmithMaterialIDElement* MaterialIDElement = static_cast<const IDatasmithMaterialIDElement*>(Element.Get());

			if (FSceneGraphId* MaterialElementIdPtr = AssetElementMapping.Find(MaterialPrefix + MaterialIDElement->GetName()))
			{
				FActionTaskFunction AssignMaterialFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
				{
					return this->AssignMaterial(Referencer, Cast<UMaterialInstanceDynamic>(Object));
				};

				AddToQueue(EQueueTask::NonAsyncQueue, { AssignMaterialFunc, *MaterialElementIdPtr, MoveTemp(Referencer) });
				TasksToComplete |= EWorkerTask::MaterialAssign;
			}
			// The value of a property has changed, process it
			else if (Element->IsA(EDatasmithElementType::KeyValueProperty))
			{
				// If it is a material's property, invalidate material and queue its processing
				if (Referencer.GetType() == EDataType::Material)
				{
					if (AssetDataList.Contains(Referencer.ElementId))
					{
						FActionTaskFunction TaskFunc = [this, ElementId = Referencer.ElementId](UObject*, const FReferencer&) -> EActionResult::Type
						{
							FAssetData& MaterialData = this->AssetDataList[ElementId];

							MaterialData.SetState(EAssetState::Unknown);

							this->ProcessMaterialData(MaterialData);

							ActionCounter.Increment();

							return EActionResult::Succeeded;
						};

						AddToQueue(EQueueTask::UpdateQueue, { MoveTemp(TaskFunc), FReferencer() } );
					}
				}
				// If it is a metadata's property, queue its application to the associated element
				else if (Referencer.GetType() == EDataType::Metadata)
				{
					//
					if (Elements.Contains(Referencer.ElementId))
					{
						FActionTaskFunction TaskFunc = [this, ElementId = Referencer.ElementId](UObject*, const FReferencer&) -> EActionResult::Type
						{
							TSharedPtr<IDatasmithMetaDataElement> MetadataElement = StaticCastSharedPtr<IDatasmithMetaDataElement>(Elements[ElementId]);

							const TSharedPtr< IDatasmithElement >& AssociatedElement = MetadataElement->GetAssociatedElement();
							if (AssociatedElement && Elements.Contains(AssociatedElement->GetNodeId()))
							{
								const FSceneGraphId AssociatedId = AssociatedElement->GetNodeId();

								if (AssetDataList.Contains(AssociatedId))
								{
									ApplyMetadata(ElementId, AssetDataList[AssociatedId].GetObject());
								}
								else if (ActorDataList.Contains(AssociatedId))
								{
									ApplyMetadata(ElementId, ActorDataList[AssociatedId].GetObject());
								}
							}

							ActionCounter.Increment();

							return EActionResult::Succeeded;
						};

						AddToQueue(EQueueTask::UpdateQueue, { MoveTemp(TaskFunc), FReferencer() } );
					}
				}
			}
		}
	}

	void FSceneImporter::IncrementalAdditions(TArray<TSharedPtr<IDatasmithElement>>& Additions, TArray<TSharedPtr<IDatasmithElement>>& Updates)
	{
		const int32 AdditionCount = Additions.Num();

		TextureElementSet.Empty(AdditionCount);
		MeshElementSet.Empty(AdditionCount);
		MaterialElementSet.Empty(AdditionCount);

		if (AdditionCount == 0)
		{
			return;
		}

		// Collect set of new textures, materials and meshes used in scene
		Elements.Reserve( Elements.Num() + AdditionCount );
		AssetDataList.Reserve( AssetDataList.Num() + AdditionCount );

		TFunction<void(TSharedPtr<IDatasmithElement>&&, EDataType)> LocalAddAsset;

		LocalAddAsset = [&](TSharedPtr<IDatasmithElement>&& Element, EDataType DataType) -> void
		{
			const FString& Prefix = DataType == EDataType::Texture ? TexturePrefix : (DataType == EDataType::Material ? MaterialPrefix : MeshPrefix);

			const FString PrefixedName = Prefix + Element->GetName();
			const FSceneGraphId ElementId = Element->GetNodeId();

			// If the new asset has the same name as an existing one, mark it as not processed
			if (this->AssetElementMapping.Contains(PrefixedName))
			{
				const FSceneGraphId ExistingElementId = AssetElementMapping[PrefixedName];
				FAssetData& ExistingAssetData = AssetDataList[ExistingElementId];

				if (!ExistingAssetData.HasState(EAssetState::PendingDelete))
				{
					UE_LOGF(LogDatasmithRuntime, Error, "Found a new %ls (%d) with the same name, %ls, as an existing one (%d).", *Prefix, ElementId, this->Elements[ExistingElementId]->GetName(), ExistingElementId);
				}

				// Add all referencers to the list of elements to update
				for (const FReferencer& Referencer : ExistingAssetData.Referencers)
				{
					if (this->AssetDataList.Contains(Referencer.ElementId))
					{
						FAssetData& ReferencerAssetData = this->AssetDataList[Referencer.ElementId];
						const bool bMustBeProcessed = ReferencerAssetData.HasState(EAssetState::Processed | EAssetState::Completed) && !ReferencerAssetData.HasState(EAssetState::PendingDelete);
						
						if (bMustBeProcessed)
						{
							ReferencerAssetData.ClearState(EAssetState::Processed);
							Updates.Add(Elements[Referencer.ElementId]);
						}
					}
					else if (this->ActorDataList.Contains(Referencer.ElementId))
					{
						this->ActorDataList[Referencer.ElementId].ClearState(EAssetState::Processed);
					}
				}

				this->AssetElementMapping[PrefixedName] = ElementId;
			}
			else
			{
				this->AssetElementMapping.Add(PrefixedName, ElementId);
			}

			this->Elements.Add(ElementId, MoveTemp(Element));

			FAssetData AssetData(ElementId, DataType);
			this->AssetDataList.Emplace(ElementId, MoveTemp(AssetData));
		};


		for (TSharedPtr<IDatasmithElement>& ElementPtr : Additions)
		{
			if (ElementPtr->IsA(EDatasmithElementType::BaseMaterial))
			{
				UE_LOGF(LogDatasmithRuntime, Log, "IncrementalAdditions: Material %ls (%d)", ElementPtr->GetName(), ElementPtr->GetNodeId());
				LocalAddAsset(MoveTemp(ElementPtr), EDataType::Material);
			}
			else if (ElementPtr->IsA(EDatasmithElementType::StaticMesh))
			{
				UE_LOGF(LogDatasmithRuntime, Log, "IncrementalAdditions: StaticMesh %ls (%d)", ElementPtr->GetName(), ElementPtr->GetNodeId());
				if (IDatasmithMeshElement* MeshElement = static_cast<IDatasmithMeshElement*>(ElementPtr.Get()))
				{
					// If resource file does not exist, add scene's resource path if valid
					if (!FPaths::FileExists(MeshElement->GetFile()) && FPaths::DirectoryExists(SceneElement->GetResourcePath()))
					{
						MeshElement->SetFile( *FPaths::Combine(SceneElement->GetResourcePath(), MeshElement->GetFile()) );
					}

					// Only add the mesh if its associated mesh file exists
					if (FPaths::FileExists(MeshElement->GetFile()))
					{
						LocalAddAsset(MoveTemp(ElementPtr), EDataType::Mesh);
					}
				}
			}
			else if (ElementPtr->IsA(EDatasmithElementType::Texture))
			{
				UE_LOGF(LogDatasmithRuntime, Log, "IncrementalAdditions: Texture %ls (%d)", ElementPtr->GetName(), ElementPtr->GetNodeId());
				if (IDatasmithTextureElement* TextureElement = static_cast<IDatasmithTextureElement*>(ElementPtr.Get()))
				{
					// If resource file does not exist, add scene's resource path if valid
					if (!FPaths::FileExists(TextureElement->GetFile()) && FPaths::DirectoryExists(SceneElement->GetResourcePath()))
					{
						TextureElement->SetFile( *FPaths::Combine(SceneElement->GetResourcePath(), TextureElement->GetFile()) );
					}

					if (FPaths::FileExists(TextureElement->GetFile()))
					{
						LocalAddAsset(MoveTemp(ElementPtr), EDataType::Texture);
					}
				}
			}
			else if (ImportOptions.bImportMetaData && ElementPtr->IsA(EDatasmithElementType::MetaData))
			{
				ProcessMetdata(StaticCastSharedPtr<IDatasmithMetaDataElement>(ElementPtr));
			}
			else if (ElementPtr->IsA(EDatasmithElementType::Actor))
			{
				UE_LOGF(LogDatasmithRuntime, Log, "IncrementalAdditions: Actor %ls (%d)", ElementPtr->GetName(), ElementPtr->GetNodeId());
			}
		}

		TasksToComplete |= EWorkerTask::SetupTasks;
	}

	void FSceneImporter::PrepareIncrementalUpdate(FUpdateContext& UpdateContext)
	{
		TasksToComplete = EWorkerTask::NoTask;

		// Update elements map with new pointers
		for (int32 Index = 0; Index < SceneElement->GetTexturesCount(); ++Index)
		{
			FSceneGraphId ElementId = SceneElement->GetTexture(Index)->GetNodeId();
			if (this->Elements.Contains(ElementId))
			{
				Elements[ElementId] = SceneElement->GetTexture(Index);
			}
		}

		for (int32 Index = 0; Index < SceneElement->GetMaterialsCount(); ++Index)
		{
			FSceneGraphId ElementId = SceneElement->GetMaterial(Index)->GetNodeId();
			if (this->Elements.Contains(ElementId))
			{
				this->Elements[ElementId] = SceneElement->GetMaterial(Index);
			}
		}

		for (int32 Index = 0; Index < SceneElement->GetMeshesCount(); ++Index)
		{
			const FSceneGraphId ElementId = SceneElement->GetMesh(Index)->GetNodeId();

			if (this->Elements.Contains(ElementId))
			{
				this->Elements[ElementId] = SceneElement->GetMesh(Index);
			}
		}

		for (int32 Index = 0; Index < SceneElement->GetActorsCount(); ++Index)
		{
			ParseScene(SceneElement->GetActor(Index), DirectLink::InvalidId,
				[this](const TSharedPtr<IDatasmithActorElement>& ActorElement, FSceneGraphId ParentId) -> void
				{
					FSceneGraphId ElementId = ActorElement->GetNodeId();
					if (this->Elements.Contains(ElementId))
					{
						this->Elements[ElementId] = ActorElement;
					}
				}
			);
		}

		for (int32 Index = 0; Index < SceneElement->GetMetaDataCount(); ++Index)
		{
			const FSceneGraphId ElementId = SceneElement->GetMetaData(Index)->GetNodeId();

			if (this->Elements.Contains(ElementId))
			{
				this->Elements[ElementId] = SceneElement->GetMetaData(Index);
			}
		}

		// Clear 'Processed' state of modified elements
		for (TSharedPtr<IDatasmithElement>& ElementPtr : UpdateContext.Updates)
		{
			const FSceneGraphId ElementId = ElementPtr->GetNodeId();

			if (AssetDataList.Contains(ElementId))
			{
				AssetDataList[ElementId].ClearState(EAssetState::Processed);
			}
			else if (ActorDataList.Contains(ElementId))
			{
				ActorDataList[ElementId].ClearState(EAssetState::Processed);
			}
		}

		// Mark assets which are about to be deleted with 'PendingDelete'
		for (DirectLink::FSceneGraphId& ElementId : UpdateContext.Deletions)
		{
			if (FAssetData* AssetData = AssetDataList.Find(ElementId))
			{
				AssetData->AddState(EAssetState::PendingDelete);
			}
			else if (FActorData* ActorData = ActorDataList.Find(ElementId))
			{
				ActorData->AddState(EAssetState::PendingDelete);
			}
		}

		// Verify that deleted assets are not referenced anymore
		for (DirectLink::FSceneGraphId& ElementId : UpdateContext.Deletions)
		{
			if (AssetDataList.Contains(ElementId))
			{
				for (FReferencer& Referencer : AssetDataList[ElementId].Referencers)
				{
					if (AssetDataList.Contains(Referencer.ElementId))
					{
						FAssetData& AssetData = AssetDataList[Referencer.ElementId];
						if (AssetData.HasState(EAssetState::Processed) && !AssetData.HasState(EAssetState::PendingDelete))
						{
							const TCHAR* ElementName = Elements[ElementId]->GetName();
							const TCHAR*  ReferencerName = Elements[Referencer.ElementId]->GetName();
							UE_LOGF(LogDatasmithRuntime, Error, "Element %ls (%d) marked for deletion but referencer %ls (%d) is neither marked for deletion or for update", ElementName, ElementId, ReferencerName, Referencer.ElementId);
							AssetData.ClearState(EAssetState::Processed);
							UpdateContext.Updates.Add(Elements[Referencer.ElementId]);
						}
					}
				}
			}
		}

		// Parse scene to mark all existing actors as not processed
		for (int32 Index = 0; Index < SceneElement->GetActorsCount(); ++Index)
		{
			ParseScene(SceneElement->GetActor(Index), DirectLink::InvalidId,
				[this](const TSharedPtr<IDatasmithActorElement>& ActorElement, FSceneGraphId ParentId) -> void
				{
					FSceneGraphId ElementId = ActorElement->GetNodeId();
					if (ActorDataList.Contains(ElementId))
					{
						ActorDataList[ElementId].ClearState(EAssetState::Processed);
					}
				}
			);
		}

		for (int32 Index = 0; Index < EQueueTask::MaxQueues; ++Index)
		{
			ActionQueues[Index].Empty();
		}
	}

	EActionResult::Type FSceneImporter::DeleteElement(FSceneGraphId ElementId)
	{
		bool bDeletionSuccessful = false;
		bool bHasSomethingToDelete = false;

		FAssetData AssetData(DirectLink::InvalidId);
		if (AssetDataList.RemoveAndCopyValue(ElementId, AssetData))
		{
			bHasSomethingToDelete = AssetData.HasState(EAssetState::Completed) && !AssetData.HasState(EAssetState::Skipped);
			bDeletionSuccessful = DeleteAsset(AssetData);
		}

		FActorData ActorData(DirectLink::InvalidId);
		if (ActorDataList.RemoveAndCopyValue(ElementId, ActorData))
		{
			bHasSomethingToDelete = ActorData.HasState(EAssetState::Completed) && !ActorData.HasState(EAssetState::Skipped);
			bDeletionSuccessful = DeleteComponent(ActorData);
		}

		ensure(bDeletionSuccessful || !bHasSomethingToDelete);

		TSharedPtr<IDatasmithElement> ElementPtr;

		return bDeletionSuccessful && Elements.RemoveAndCopyValue(ElementId, ElementPtr) ? EActionResult::Succeeded : EActionResult::Failed;
	}

	bool FSceneImporter::DeleteAsset(FAssetData& AssetData)
	{
		FString AssetPrefixedName;

		const FSceneGraphId ElementId = AssetData.ElementId;
		const EDataType DataType(AssetData.Type);

		if (DataType == EDataType::Texture)
		{
			AssetPrefixedName = TexturePrefix + Elements[ElementId]->GetName();
		}
		else if (DataType == EDataType::Material || DataType == EDataType::PbrMaterial)
		{
			// If asset is a material and it references textures, remove it from the textures' list of referencers
			FTextureCallback TextureCallback;
			TextureCallback = [this, ElementId](const FString& TextureNamePrefixed, int32 PropertyIndex)->void
			{
				this->RemoveFromReferencer(this->AssetElementMapping.Find(TextureNamePrefixed),ElementId);
			};

			TSharedPtr< IDatasmithElement >& Element = Elements[ AssetData.ElementId ];

			if( Element->IsA( EDatasmithElementType::UEPbrMaterial ) )
			{
				ProcessMaterialElement(static_cast<IDatasmithUEPbrMaterialElement*>(Element.Get()), TextureCallback);
			}
			else if( Element->IsA( EDatasmithElementType::MaterialInstance ) )
			{
				ProcessMaterialElement(StaticCastSharedPtr<IDatasmithMaterialInstanceElement>(Element), TextureCallback);
			}

			AssetPrefixedName = MaterialPrefix + Element->GetName();
		}
		else if (DataType == EDataType::Mesh)
		{
			// If asset is a mesh and it references materials, remove it from the materials' list of referencers
			TSharedPtr< IDatasmithMeshElement > MeshElement = StaticCastSharedPtr< IDatasmithMeshElement >(Elements[ElementId]);

			for (int32 Index = 0; Index < MeshElement->GetMaterialSlotCount(); Index++)
			{
				if (const IDatasmithMaterialIDElement* MaterialIDElement = MeshElement->GetMaterialSlotAt(Index).Get())
				{
					const FString MaterialPathName(MaterialIDElement->GetName());

					if (!MaterialPathName.StartsWith(TEXT("/")))
					{
						RemoveFromReferencer(AssetElementMapping.Find(MaterialPrefix + MaterialPathName), ElementId);
					}
				}
			}

			AssetPrefixedName = MeshPrefix + MeshElement->GetName();
		}

		ensure(AssetElementMapping.Contains(AssetPrefixedName));

		// ElementId may mismatch if new object of same name but new id was added
		if (AssetElementMapping[AssetPrefixedName] == AssetData.ElementId)
		{
			AssetElementMapping.Remove(AssetPrefixedName);
		}

		if (UObject* Asset = AssetData.Object.Get())
		{
			AssetData.Object.Reset();
			FAssetRegistry::UnregisterAssetData(Asset, SceneKey, AssetData.ElementId);
		}

		return true;
	}

	void FSceneImporter::ProcessMetdata(const TSharedPtr<IDatasmithMetaDataElement>& MetadataElement)
	{
		// Process metadata only if it has properties and the associated element is tracked
		if (MetadataElement && MetadataElement->GetPropertiesCount() > 0)
		{
			const TSharedPtr< IDatasmithElement >& AssociatedElement = MetadataElement->GetAssociatedElement();
			const FSceneGraphId AssociatedId = AssociatedElement ? AssociatedElement->GetNodeId() : DirectLink::InvalidId;

			if (Elements.Contains(AssociatedId))
			{
				if (AssetDataList.Contains(AssociatedId))
				{
					AssetDataList[AssociatedId].MetadataId = MetadataElement->GetNodeId();
				}
				else if (ActorDataList.Contains(AssociatedId))
				{
					FActorData& ActorData = ActorDataList[AssociatedId];

					ActorData.MetadataId = MetadataElement->GetNodeId();

					// Record task to assign metadata if the actor has already been created.
					// This happens for 'simple' actor, i.e. container of child actors.
					if (ActorData.HasState(EAssetState::Completed))
					{
						FActionTaskFunction ApplyMetadataFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
						{
							if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object))
							{
								this->ApplyMetadata(Referencer.GetId(), SceneComponent);
								return EActionResult::Succeeded;
							}

							return EActionResult::Failed;
						};

						AddToQueue(EQueueTask::NonAsyncQueue, { ApplyMetadataFunc, ActorData.ElementId, { EDataType::Metadata, ActorData.MetadataId, 0 } });
						TasksToComplete |= EWorkerTask::ComponentFinalize;
					}
				}

				Elements.Add(MetadataElement->GetNodeId(), MetadataElement);
			}
			else if (AssociatedElement == SceneElement)
			{
				Elements.Add(MetadataElement->GetNodeId(), MetadataElement);
				ApplyMetadata(MetadataElement->GetNodeId(), RootComponent.Get());
			}
		}
	}

	// Logic borrowed from FDatasmithImporter::ImportMetaDataForObject
	void FSceneImporter::ApplyMetadata(FSceneGraphId MetadataId, UObject* Object)
	{
		if ( !Object || !Object->GetClass()->ImplementsInterface( UInterface_AssetUserData::StaticClass() ) || MetadataId == DirectLink::InvalidId)
		{
			return;
		}

		if (IDatasmithMetaDataElement* MetadataElement = static_cast<IDatasmithMetaDataElement*>(Elements[MetadataId].Get()))
		{
			if (IInterface_AssetUserData* AssetUserData = Cast< IInterface_AssetUserData >(Object))
			{
				UDatasmithAssetUserData* DatasmithUserData = AssetUserData->GetAssetUserData< UDatasmithAssetUserData >();

				if ( !DatasmithUserData )
				{
					DatasmithUserData = NewObject<UDatasmithAssetUserData>( Object, NAME_None, RF_Public | RF_Transactional );
					AssetUserData->AddAssetUserData( DatasmithUserData );
				}

				UDatasmithAssetUserData::FMetaDataContainer MetaData;

				const int32 PropertiesCount = MetadataElement->GetPropertiesCount();
				MetaData.Reserve( PropertiesCount + 1 );

				// Add associated element's unique id
				MetaData.Add( UDatasmithAssetUserData::UniqueIdMetaDataKey, MetadataElement->GetAssociatedElement()->GetName() );

				// Add Datasmith metadata's properties
				for ( int32 PropertyIndex = 0; PropertyIndex < PropertiesCount; ++PropertyIndex )
				{
					const TSharedPtr<IDatasmithKeyValueProperty>& Property = MetadataElement->GetProperty( PropertyIndex );
					MetaData.Add( Property->GetName(), Property->GetValue() );

					DependencyList.Add(Property->GetNodeId(), { EDataType::Metadata, MetadataId, 0xffff });
				}

				MetaData.KeySort(FNameLexicalLess());

				DatasmithUserData->MetaData = MoveTemp( MetaData );
			}
		}
	}

	void FSceneImporter::RemoveFromReferencer(FSceneGraphId* AssetIdPtr, FSceneGraphId ReferencerId)
	{
		if (AssetIdPtr)
		{
			TArray<FReferencer>& Referencers = AssetDataList[*AssetIdPtr].Referencers;

			for (int32 Index = 0; Index < Referencers.Num(); ++Index)
			{
				if (Referencers[Index].ElementId == ReferencerId)
				{
					Referencers.RemoveAt(Index, EAllowShrinking::No);
					return;
				}
			}
		}
	}

} // End of namespace DatasmithRuntime
