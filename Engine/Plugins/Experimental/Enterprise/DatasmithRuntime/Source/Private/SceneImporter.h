// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DatasmithRuntime.h"

#include "PixelFormat.h"
#include "Tickable.h"
#include "RHITypes.h"

struct FMeshDescription;

class IDatasmithActorElement;
class IDatasmithElement;
class IDatasmithCameraActorElement;
class IDatasmithLightActorElement;
class IDatasmithMetaDataElement;
class IDatasmithMeshActorElement;
class IDatasmithScene;
class IDatasmithTranslator;
class UMaterial;
class UMaterialInstanceDynamic;
class ULightComponent;
class USceneComponent;
class UStaticMesh;
class UTexture2D;
class UTextureLightProfile;

struct FUpdateContext;

#if WITH_EDITOR
#define LIVEUPDATE_TIME_LOGGING
//#define ASSET_DEBUG
#endif

DECLARE_MULTICAST_DELEGATE_OneParam(FOnStaticMeshComplete, UStaticMesh*)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialComplete, UMaterialInstanceDynamic*)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTextureComplete, UTexture2D*)

namespace DatasmithRuntime
{
#ifdef LIVEUPDATE_TIME_LOGGING
	class Timer
	{
	public:
		Timer(double InTimeOrigin, const char* InText);
		~Timer();

	private:
		double TimeOrigin;
		double StartTime;
		FString Text;
	};
#endif

	typedef DirectLink::FSceneGraphId FSceneGraphId;

	using FParsingCallback = TFunction<void(const TSharedPtr<IDatasmithActorElement>&, FSceneGraphId)>;

	struct CaseSensitive_KeyFuncs : BaseKeyFuncs<TPair<FString, FSceneGraphId>, FString, false>
	{
		static FORCEINLINE const FString&   GetSetKey(const TPair<FString, FSceneGraphId>& Element) { return Element.Key; }
		static FORCEINLINE uint32           GetKeyHash(const FString& Key) { return GetTypeHash(Key); }
		static FORCEINLINE bool             Matches(const FString& A, const FString& B) { return A.Equals(B, ESearchCase::CaseSensitive); }
	};

	using FCaseSensitiveMap = TMap<FString, FSceneGraphId, FDefaultSetAllocator, CaseSensitive_KeyFuncs>;

	enum class EWorkerTask : uint32
	{
		NoTask                  = 0x00000000,
		CollectSceneData        = 0x00000001,
		UpdateElement           = 0x00000002,
		ResetScene              = 0x00000004,
		SetupTasks              = 0x00000008,

		MeshCreate              = 0x00000010,
		MaterialCreate          = 0x00000020,
		TextureLoad             = 0x00000040,
		TextureCreate           = 0x00000080,

		MeshComponentCreate     = 0x00000100,
		LightComponentCreate    = 0x00000200,
		ComponentFinalize       = 0x00000400,
		ComputeCollision        = 0x00000800,

		MaterialAssign          = 0x00001000,
		TextureAssign           = 0x00002000,

		DeleteComponent         = 0x00010000,
		DeleteAsset             = 0x00020000,
		GarbageCollect          = 0x00040000,

		NonAsyncTasks           = LightComponentCreate | MeshComponentCreate | MaterialAssign | TextureCreate | TextureAssign | ComponentFinalize,
		DeleteTasks             = DeleteComponent | DeleteAsset,

		AllTasks                = 0xffffffff
	};

	ENUM_CLASS_FLAGS(EWorkerTask);

	enum class EAssetState : uint8
	{
		Unknown        = 0x00,
		Processed      = 0x01,
		Completed      = 0x02,
		Skipped        = 0x04,
		Building       = 0x08,
		PendingDelete  = 0x10,
		AllStates      = 0xff
	};

	ENUM_CLASS_FLAGS(EAssetState);

	// Order is important as it reflects dependency: bottom to top
	enum class EDataType : uint8
	{
		None        = 0,
		Texture     = 1,
		Material    = 2,
		PbrMaterial = 3,
		Mesh        = 4,
		Actor       = 5,
		MeshActor   = 6,
		LightActor  = 7,
		Metadata    = 8,
	};

	/**
	 * Utility structure to track elements referencing an asset
	 */
	struct FReferencer
	{
		uint32 Type:4;
		uint32 ElementId:28; // Assuming 2^28 should be plenty to store index of referencers
		uint16 Slot; // Assuming 65536 should be plenty

		FReferencer(EDataType InType, FSceneGraphId InIndex, uint16 InSlot)
			: Type( uint8(InType) )
			, ElementId( uint32(InIndex) )
			, Slot( InSlot )
		{
		}

		FReferencer(EDataType InType, FSceneGraphId InIndex)
			: Type( uint8(InType) )
			, ElementId( uint32(InIndex) )
			, Slot( 0 )
		{
		}

		FReferencer(FSceneGraphId InIndex)
			: Type( 0 )
			, ElementId( uint32(InIndex) )
			, Slot( 0 )
		{
		}

		FReferencer()
			: Type( 0 )
			, ElementId( 0 )
			, Slot( 0 )
		{
		}

		EDataType GetType() const { return EDataType(Type); }
		FSceneGraphId GetId() const { return (FSceneGraphId)ElementId; }
	};

	FORCEINLINE bool operator==(const FReferencer& Lhs, const FReferencer& Rhs)
	{
		return Lhs.ElementId == Rhs.ElementId && Lhs.Slot == Rhs.Slot;
	}


	typedef std::atomic<EAssetState> FDataState;

	/**
	 * Utility structure to hold onto information used during the import process
	 */
	struct FBaseData
	{
		/** Identifier of the associated Datasmith element */
		FSceneGraphId ElementId;

		/** Identifier of the associated metadata element */
		FSceneGraphId MetadataId;

		EDataType Type;

		/** UObject associated with the element */
		TWeakObjectPtr<UObject> Object;

		/** State in which the element is within the import process */
		FDataState DataState;

		/** Array of elements referencing this element */
		TArray< FReferencer > Referencers;

		FBaseData(FSceneGraphId InElementId, EDataType InType = EDataType::None)
			: ElementId(InElementId)
			, MetadataId(DirectLink::InvalidId)
			, Type(InType)
		{
			DataState.store(EAssetState::Unknown);
		}

		FBaseData(const FBaseData& Other)
		{
			ElementId = Other.ElementId;
			MetadataId = Other.MetadataId;
			Type = Other.Type;
			Object = Other.Object;
			DataState.store(Other.DataState.load());
			Referencers = Other.Referencers;
		}

		// Non-assignable due to DataState atomic member
		FBaseData& operator=(const FBaseData& Other) = delete;

		bool HasState(EAssetState Value) const
		{
			return !!(DataState & Value);
		}

		void AddState(EAssetState Value)
		{
			DataState.store(DataState | Value);
		}

		void ClearState(EAssetState Value)
		{
			DataState.store(DataState & ~Value);
		}

		void SetState(EAssetState Value)
		{
			DataState.store(Value);
		}

		template<typename T = UObject>
		T* GetObject() const
		{
			return Cast< T >(Object.Get());
		}
	};

	/**
	 * Utility structure to hold onto additional information used for assets
	 */
	struct FAssetData : public FBaseData
	{
		/** Build settings requirements defined by materials, used by static meshes */
		int32 Requirements = 0;

		/** Hash of associated element used to prevent the duplication of assets */
		uint32 Hash = 0;

		/** Hash of potential resource of associated element used to prevent recreation of assets */
		uint32 ResourceHash = 0;

		FAssetData(FSceneGraphId InElementId, EDataType InType = EDataType::None)
			: FBaseData(InElementId, InType)
			, Requirements(-1)
		{
		}

		// Copy constructor intentionally copies the MetadataId member whereas the assignment operator doesn't - is this intentional?
		FAssetData(const FAssetData&) = default;

		FAssetData& operator=(const FAssetData& Source)
		{
			ElementId = Source.ElementId;
			Type = Source.Type;
			Object = Source.Object;
			DataState.store(Source.DataState.load());
			Referencers = Source.Referencers;

			Requirements = Source.Requirements;
			Hash = Source.Hash;

			return *this;
		}

		static FAssetData EmptyAsset;
	};

	/**
	 * Utility structure to hold onto additional information used for actors
	 */
	struct FActorData : public FBaseData
	{
		/** Index of parent actor in FSceneImporter's array of FActorData */
		FSceneGraphId ParentId;

		/** Transform relative to parent */
		FTransform RelativeTransform;

		/** Transform relative to world */
		FTransform WorldTransform;

		/** Index of referenced mesh (mesh actor) or texture (light actor) */
		int32 AssetId;

		FActorData(FSceneGraphId InElementId)
			: FBaseData(InElementId, EDataType::Actor)
			, ParentId(DirectLink::InvalidId)
			, AssetId(INDEX_NONE)
		{
		}

		FActorData(FSceneGraphId InElementId, FSceneGraphId InParentID)
			: FBaseData(InElementId, EDataType::Actor)
			, ParentId(InParentID)
			, AssetId(INDEX_NONE)
		{
		}

		// Copy constructor intentionally copies the MetadataId member whereas the assignment operator doesn't - is this intentional?
		FActorData(const FActorData&) = default;

		FActorData& operator=(const FActorData& Source)
		{
			ElementId = Source.ElementId;
			Type = Source.Type;
			Object = Source.Object;
			DataState.store(Source.DataState.load());
			Referencers = Source.Referencers;

			ParentId = Source.ParentId;
			RelativeTransform = Source.RelativeTransform;
			WorldTransform = Source.WorldTransform;
			AssetId = Source.AssetId;

			return *this;
		}
	};

	/**
	 * Texture assets can only be created and built on the main thread
	 * Consequently, their creation has been divided in two steps:
	 *		- Asynchronously load the data of the texture
	 *		- At each tick create a texture from its data until all required textures are done
	 */ 
	struct FTextureData
	{
		EPixelFormat PixelFormat;
		int32 Width;
		int32 Height;
		uint32 Pitch;
		int16 BytesPerPixel;
		FUpdateTextureRegion2D Region;
		uint8* ImageData;
		// For IES profile
		float Brightness;
		float TextureMultiplier;

		FTextureData()
			: PixelFormat(EPixelFormat::PF_Unknown)
			, Width(0)
			, Height(0)
			, Pitch(0)
			, BytesPerPixel(0)
			, Region(0,0,0,0,0,0)
			, ImageData(nullptr)
			, Brightness(-FLT_MAX)
			, TextureMultiplier(-FLT_MAX)
		{
		}
	};

	namespace EActionResult
	{
		enum Type : uint8
		{
			Unknown			= 0,
			Succeeded		= 1,
			Failed			= 2,
			Retry			= 3,
		};
	};

	using FActionTaskFunction = TFunction<EActionResult::Type(UObject* Object, const FReferencer& Referencer)>;

	class FActionTask
	{
	public:
		~FActionTask()
		{
		}

		FActionTask()
			: ElementId(DirectLink::InvalidId)
		{
		}

		FActionTask(FActionTaskFunction&& Function, const FReferencer& InReferencer)
			: ElementId(DirectLink::InvalidId)
			, Referencer(InReferencer)
			, ActionFunc(MoveTemp(Function))
		{
		}

		FActionTask(const FActionTaskFunction& Function, const FReferencer& InReferencer)
			: ElementId(DirectLink::InvalidId)
			, Referencer(InReferencer)
			, ActionFunc(Function)
		{
		}

		FActionTask(FActionTaskFunction&& Function, FSceneGraphId InElementId, const FReferencer& InReferencer)
			: ElementId(InElementId)
			, Referencer(InReferencer)
			, ActionFunc(MoveTemp(Function))
		{
		}

		FActionTask(const FActionTaskFunction& Function, FSceneGraphId InElementId, const FReferencer& InReferencer)
			: ElementId(InElementId)
			, Referencer(InReferencer)
			, ActionFunc(Function)
		{
		}

		FSceneGraphId GetElementId() const { return ElementId; }

		const FReferencer& GetReferencer() const { return Referencer; }

		EActionResult::Type Execute(FBaseData& ElementData)
		{
			return ElementData.HasState(EAssetState::Completed) ? ActionFunc(ElementData.GetObject<>(), Referencer) : EActionResult::Retry;
		}

	private:
		FSceneGraphId ElementId;
		FReferencer Referencer;
		FActionTaskFunction ActionFunc;
	};

	extern const FString TexturePrefix;
	extern const FString MaterialPrefix;
	extern const FString MeshPrefix;


	enum EQueueTask
	{
		UpdateQueue      = 0,
		MeshQueue        = 1,
		MaterialQueue    = 2,
		TextureQueue     = 3,
		NonAsyncQueue    = 4,
		DeleteCompQueue  = 5,   // Index of queue to delete components
		DeleteAssetQueue = 6,   // Index of queue to delete assets
		MaxQueues        = 7,
	};

	/**
	 * Helper class to incrementally load a Datasmith scene at runtime
	 * The creation of the assets and components is incrementally done on the tick of the object.
	 * At each tick, a budget of 10 ms is allocated to perform as much tasks as possible
	 * The load process is completely interruptible.
	 * Datasmith actor elements are added as component to the root component of the associated ADatasmithRuntimeActor.
	 * @note: Only assets used by Datasmith actor elements are created.
	 * The creation is phased as followed:
	 *		- Collection of the assets and actors to be added
	 *		- Launch of asynchronous build of static meshes
	 *		- Launch of asynchronous load of images used by textures
	 *		- Creation of Materials, textures, components and resolution of referencing
	 *		  (i.e material assignment, ...) are synchronously done on the Game thread.
	 */
	class FSceneImporter : public FTickableGameObject
	{
	public:
		explicit FSceneImporter(ADatasmithRuntimeActor* InDatasmithRuntimeActor);

		virtual ~FSceneImporter();

		/**
		 * Start the import process of a scene
		 * @param InSceneElement: Datasmith scene to import
		 * @param Options: Options to use for the import process
		 */
		void StartImport(TSharedRef< IDatasmithScene > InSceneElement, const FDatasmithRuntimeImportOptions& Options);

		/** Abort the on going import process then delete all created assets and actors */
		void Reset(bool bIsNewScene);

		/** Returns the Datasmith element associated to a given asset name */
		TSharedPtr< IDatasmithElement > GetElementFromName(const FString& PrefixedName)
		{
			FSceneGraphId* ElementIdPtr = AssetElementMapping.Find(PrefixedName);

			if (ElementIdPtr && Elements.Contains(*ElementIdPtr) )
			{
				return Elements[*ElementIdPtr];
			}

			return {};
		}

		/** Start the incremental update of the elements contained in the given context */
		bool IncrementalUpdate(TSharedRef< IDatasmithScene > InSceneElement, FUpdateContext& UpdateContext);

		void SetTranslator(const TSharedPtr<IDatasmithTranslator>& InTranslator) { Translator = InTranslator; }

	protected:
		//~ Begin FTickableEditorObject interface
		virtual void Tick(float DeltaSeconds) override;
		virtual bool IsTickable() const override { return RootComponent.IsValid() && TasksToComplete != EWorkerTask::NoTask; }
		virtual TStatId GetStatId() const override;
		//~ End FTickableEditorObject interface

	private:

		/** Delete all the assets and components created during the previous import process */
		bool DeleteData();

		/** Delete the asset or component associated with the Datasmith element associated with the ElementId */
		EActionResult::Type DeleteElement(FSceneGraphId ElementId);

		/** Delete the component created from the given FActorData */
		bool DeleteComponent(FActorData& ActorData);

		/** Delete the asset created from the given FAssetData */
		bool DeleteAsset(FAssetData& AssetData);

		/** Remove given referencer from the list of referencers of the asset */
		void RemoveFromReferencer(FSceneGraphId* AssetIdPtr, FSceneGraphId ReferencerId);

		/**
		 * Creates the FAssetData and FActorData required to import the associated Datasmith scene
		 * This is the first task  after StartImport has been called
		 */
		void CollectSceneData();

		/** Sets up all counters and data required to proceed with a full import or an incremental update */
		void SetupTasks();

		void PrepareIncrementalUpdate(FUpdateContext& UpdateContext);

		void IncrementalAdditions(TArray<TSharedPtr<IDatasmithElement>>& Additions, TArray<TSharedPtr<IDatasmithElement>>& Updates);

		void IncrementalModifications(TArray<TSharedPtr<IDatasmithElement>>& Modifications);

		void IncrementalDeletions(TArray<DirectLink::FSceneGraphId>& Deletions);

		/** Add an FAssetData object associated with the element's id to the map */
		void AddAsset(TSharedPtr<IDatasmithElement>&& ElementPtr, const FString& Prefix, EDataType InType = EDataType::None);

		/**
		 * Recursive helper method to visit the children of an Datasmith actor element
		 * @param ActorElement: Datasmith actor element to visit
		 * @param ParentId: Identifier of the incoming actor's parent
		 * @param Callback: function called on incoming actor
		 */
		void ParseScene(const TSharedPtr<IDatasmithActorElement>& ActorElement, FSceneGraphId ParentId, FParsingCallback Callback);

		/** Add and populate the FActorData created for the incoming Datasmith actor element */
		void ProcessActorElement(const TSharedPtr< IDatasmithActorElement >& ActorElement, FSceneGraphId ParentId);

		/** Populate the FActorData created for the incoming Datasmith mesh actor element */
		bool ProcessMeshActorData(FActorData& ActorData, IDatasmithMeshActorElement* MeshActorElement);

		/** Populate the FActorData created for the incoming Datasmith light actor element */
		bool ProcessLightActorData(FActorData& ActorData, IDatasmithLightActorElement* LightActorElement);

		/** Populate the FActorData created for the incoming Datasmith camera actor element */
		bool ProcessCameraActorData(FActorData& ActorData, IDatasmithCameraActorElement* CameraActorElement);

		/**
		 * Populate the FAssetData based on the associated Datasmith mesh element
		 * @note: A static mesh is created at this stage to be used in the asynchronous build process
		 */
		bool ProcessMeshData(FAssetData& MeshData);

		/** Populate the FAssetData based on the associated Datasmith material element */
		void ProcessMaterialData(FAssetData& MaterialData);

		/** Create the FAssetData based on the associated Datasmith material element */
		EActionResult::Type ProcessMaterial(FSceneGraphId MaterialId);

		/** Add and populate the FActorData created for the incoming Datasmith actor element */
		void CreateActorComponent(FActorData& ActorData, const TSharedPtr< IDatasmithActorElement >& ActorElement);

		/** Add and populate a FTextureData associated with the incoming Datasmith texture element */
		void ProcessTextureData(FSceneGraphId TextureId);

		/** Asynchronous build of a static mesh */
		bool CreateStaticMesh(FSceneGraphId ElementId);

		/** Update StaticMaterials array based on mesh description and mesh element */
		void FillStaticMeshMaterials(FAssetData& MeshData, TArray< FMeshDescription >& MeshDescriptions);

		/** Update StaticMaterials array based on mesh description and mesh element */
		void UpdateStaticMeshMaterials(FAssetData& MeshData);

		/** Create and add a static mesh component to the root component */
		EActionResult::Type CreateMeshComponent(FSceneGraphId ActorId, UStaticMesh* StaticMesh);

		/** Assign the given material to the object associated to the referencer, static mesh or static mesh component */
		EActionResult::Type AssignMaterial(const FReferencer& Referencer, UMaterialInstanceDynamic* Material);

		/** Asynchronous load of the image or IES file required to build a texture */
		bool LoadTexture(FSceneGraphId ElementId);

		/** Create the UTexture object associated with the given element identifier */
		EActionResult::Type CreateTexture(FSceneGraphId ElementId);

		/** Assign the given 2D texture to the object associated to the referencer, a material */
		EActionResult::Type AssignTexture(const FReferencer& Referencer, UTexture2D* Texture);

		/** Assign the given IES texture to the object associated to the referencer, a light component */
		EActionResult::Type AssignProfileTexture(const FReferencer& Referencer, UTextureLightProfile* TextureProfile);

		/** Create and add a light component to the root component based on the type of the identified Datasmith element */
		EActionResult::Type CreateLightComponent(FSceneGraphId ActorId);

		void FinalizeComponent(FActorData& ActorData);

		/** Calls when an element on which an asset or actor depends on has changed */
		void ProcessDependency(const TSharedPtr<IDatasmithElement>& Element);

		/** Add a new task to the given queue */
		void AddToQueue(int32 WhichQueue, FActionTask&& ActionTask)
		{
			++QueuedTaskCount;
			const FSceneGraphId ElementId = ActionTask.GetElementId();
			if (ElementId != DirectLink::InvalidId)
			{
				TArray<FReferencer>& Referencers = AssetDataList.Contains(ElementId) ? AssetDataList[ElementId].Referencers : ActorDataList[ElementId].Referencers;
				Referencers.AddUnique(ActionTask.GetReferencer());
			}
			ActionQueues[WhichQueue].Enqueue(MoveTemp(ActionTask));
		}

		/** Helper method to dequeue a given queue for a given amount of time */
		void ProcessQueue(int32 Which, double EndTime, EWorkerTask TaskCompleted = EWorkerTask::NoTask, EWorkerTask TaskFollowing = EWorkerTask::NoTask)
		{
			FActionTask ActionTask;
			while (FPlatformTime::Seconds() < EndTime)
			{
				if (!ActionQueues[Which].Dequeue(ActionTask))
				{
					TasksToComplete &= ~TaskCompleted;
					TasksToComplete |= TaskFollowing;

					break;
				}

				ensure(DirectLink::InvalidId == ActionTask.GetElementId());
				ActionTask.Execute(FAssetData::EmptyAsset);
			}
		}

		FActorData& FindOrAddActorData(const TSharedPtr< IDatasmithActorElement >& ActorElement);

		/** Add metadata to the tracking system if it applies to any tracked asset or actor or the scene */
		void ProcessMetdata(const TSharedPtr< IDatasmithMetaDataElement >& MetadataElement);

		/** Apply metadata to input object if applicable */
		void ApplyMetadata(FSceneGraphId MetadataId, UObject* Object);

	private:
		/** DatasmithRuntime actor associated with this importer */
		TWeakObjectPtr<USceneComponent> RootComponent;

		/** IDatasmithScene associated with DatasmithRuntime actor */
		TSharedPtr<IDatasmithScene> SceneElement;

		TSharedPtr<IDatasmithTranslator> Translator;

		/** Map of all elements in the IDatasmithScene */
		TMap< FSceneGraphId, TSharedPtr< IDatasmithElement > > Elements;

		/** Mapping between prefixed asset element's name and index of element in flatten element list */
		FCaseSensitiveMap AssetElementMapping;

		/** Mapping between Datasmith element's identifiers and their associated FAssetData object */
		TMap< FSceneGraphId, FAssetData > AssetDataList;

		/** Mapping between Datasmith actor element's identifiers and their associated FActorData object */
		TMap< FSceneGraphId, FActorData > ActorDataList;

		/** Mapping between Datasmith texture element's identifiers and their associated FActorData object */
		TMap< FSceneGraphId, FTextureData > TextureDataList;

		/** Mapping between Datasmith asset elements and their dependent elements */
		TMap< FSceneGraphId, FReferencer > DependencyList;

		/** Set of Datasmith mesh element's identifiers to process */
		TSet< FSceneGraphId > MeshElementSet;

		/** Set of Datasmith material element's identifiers to process */
		TSet< FSceneGraphId > MaterialElementSet;

		/** Set of Datasmith texture element's identifiers to process */
		TSet< FSceneGraphId > TextureElementSet;

		/** Mapping between Datasmith mesh element's identifiers and their lightmap weights */
		TMap< FSceneGraphId, float > LightmapWeights;

		/** Array of queues dequeued during the import process */
		TQueue< FActionTask, EQueueMode::Mpsc > ActionQueues[EQueueTask::MaxQueues];

		/** Array of asynchronous tasks started during the import */
		TArray<TFuture<bool>> OnGoingTasks;

		/** Flag used to properly sequence the import process */
		EWorkerTask TasksToComplete;

		/** Indicated a incremental update has been requested */
		uint8 bIncrementalUpdate:1;

		/** Specifies options to use during the import */
		FDatasmithRuntimeImportOptions ImportOptions;

		/** Future for asynchronous task to collect metadata */
		TFuture<void> MetadataCollect;

		/** Miscellaneous counters used to report progress */
		float& OverallProgress;
		FThreadSafeCounter ActionCounter;
		double ProgressStep;
		int32 QueuedTaskCount;

		/** GUID of the last scene imported */
		FGuid LastSceneGuid;
		uint32 LastSceneKey;
		uint32 SceneKey;

	#ifdef LIVEUPDATE_TIME_LOGGING
		double GlobalStartTime;
	#endif
	};

} // End namespace DatasmithRuntime

#ifdef LIVEUPDATE_TIME_LOGGING
#define LIVEUPDATE_LOG_TIME Timer( GlobalStartTime, __func__ )
#else
#define LIVEUPDATE_LOG_TIME
#endif
