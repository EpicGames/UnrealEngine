// Copyright Epic Games, Inc. All Rights Reserved.


#include "DatasmithRuntimeAuxiliaryData.h"

#include "DatasmithRuntimeUtils.h"
#include "LogCategory.h"

#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"

#include "Async/Async.h"
#include "Engine/TextureLightProfile.h"
#include "IESConverter.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "RHI.h"

#if WITH_EDITORONLY_DATA
#include "EditorFramework/AssetImportData.h"
#endif

namespace DatasmithRuntime
{
	using FDataCleanupFunc = TFunction<void(uint8*, const FUpdateTextureRegion2D*)>;

	bool GetTextureDataForIes(const TCHAR* Filename, FTextureData& TextureData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DatasmithRuntime::GetTextureDataForIes);

		TArray<uint8> Buffer;
		if (!(FFileHelper::LoadFileToArray(Buffer, Filename) && Buffer.Num() > 0))
		{
			return false;
		}

		// checks for .IES extension to avoid wasting loading large assets just to reject them during header parsing
		FIESConverter IESConverter(Buffer.GetData(), Buffer.Num());

		if(IESConverter.IsValid())
		{
			TextureData.Width = IESConverter.GetWidth();
			TextureData.Height = IESConverter.GetHeight();
			TextureData.Brightness = IESConverter.GetBrightness();
			TextureData.BytesPerPixel = 8; // RGBA16F
			TextureData.Pitch = TextureData.Width * TextureData.BytesPerPixel;
			TextureData.TextureMultiplier = IESConverter.GetMultiplier();

			const TArray<uint8>& RAWData = IESConverter.GetRawData();

			TextureData.ImageData = (uint8*)FMemory::Malloc(RAWData.Num() * sizeof(uint8), 0x20);
			FMemory::Memcpy(TextureData.ImageData, RAWData.GetData(), RAWData.Num() * sizeof(uint8));

			return true;
		}

		return false;
	}

	UTexture2D* CreateImageTexture(UTexture2D* Texture2D, FTextureData& TextureData, IDatasmithTextureElement* TextureElement, FDataCleanupFunc& DataCleanupFunc)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::CreateImageTexture);

		if (Texture2D == nullptr)
		{
			Texture2D = UTexture2D::CreateTransient(TextureData.Width, TextureData.Height, TextureData.PixelFormat);
			if (!Texture2D)
			{
				return nullptr;
			}

			FString TextureName = FString::Printf(TEXT("T_%s_%d"), TextureElement->GetName(), TextureElement->GetNodeId());
#ifdef ASSET_DEBUG
			UPackage* Package = CreatePackage(*FPaths::Combine( TEXT("/Game/Runtime/Textures"), TextureName));
			RenameObject(Texture2D, *TextureName, Package);
			Texture2D->SetFlags(RF_Public);
#else
			RenameObject(Texture2D, *TextureName);
#endif
		}

#if WITH_EDITORONLY_DATA
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(TextureElement->GetFile()));
		Texture2D->AssetImportData->SourceData = MoveTemp(Info);

		const float RGBCurve = TextureElement->GetRGBCurve();
		if (FMath::IsNearlyEqual(RGBCurve, 1.0f) == false && RGBCurve > 0.f)
		{
			Texture2D->AdjustRGBCurve = RGBCurve;
		}
#endif

		Texture2D->SRGB = TextureElement->GetSRGB() == EDatasmithColorSpace::sRGB;

		// Ensure there's no compression (we're editing pixel-by-pixel)
		Texture2D->CompressionSettings = TextureCompressionSettings::TC_VectorDisplacementmap;

		// Update the texture with these new settings
		Texture2D->UpdateResource();

		// The content of the texture has changed, update it
		if (TextureData.ImageData != nullptr)
		{
			TextureData.Region = FUpdateTextureRegion2D(0, 0, 0, 0, TextureData.Width, TextureData.Height);

			Texture2D->UpdateTextureRegions(0, 1, &TextureData.Region, TextureData.Pitch, TextureData.BytesPerPixel, TextureData.ImageData, DataCleanupFunc );
		}

		return Texture2D;
	}

	UTextureLightProfile* CreateIESTexture(UTextureLightProfile* Texture, FTextureData& TextureData, IDatasmithTextureElement* TextureElement, FDataCleanupFunc& DataCleanupFunc)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::CreateIESTexture);

		if (Texture == nullptr)
		{
			Texture = NewObject<UTextureLightProfile>();
			if (!Texture)
			{
				return nullptr;
			}

			FString TextureName = FString::Printf(TEXT("T_%s_%d"), TextureElement->GetName(), TextureElement->GetNodeId());
			TextureName = FDatasmithUtils::SanitizeObjectName(TextureName);
#ifdef ASSET_DEBUG
			UPackage* Package = CreatePackage(*FPaths::Combine( TEXT("/Game/Runtime/Textures"), TextureName));
			RenameObject(Texture, *TextureName, Package);
			Texture->SetFlags(RF_Public);
#else
			RenameObject(Texture, *TextureName);
#endif
		}

		// TextureData.ImageData should not be null
		ensure(TextureData.ImageData);

#if WITH_EDITORONLY_DATA
		FAssetImportInfo Info;
		Info.Insert(FAssetImportInfo::FSourceFile(TextureElement->GetFile()));
		Texture->AssetImportData->SourceData = MoveTemp(Info);
#endif

#if WITH_EDITOR
		Texture->Source.Init(
			TextureData.Width,
			TextureData.Height,
			/*NumSlices=*/ 1,
			1,
			TSF_RGBA16F,
			TextureData.ImageData
		);

		DataCleanupFunc(nullptr, nullptr);
#endif

		Texture->LODGroup = TEXTUREGROUP_IESLightProfile;
		Texture->AddressX = TA_Clamp;
		Texture->AddressY = TA_Clamp;
		Texture->CompressionSettings = TC_HDR;
#if WITH_EDITORONLY_DATA
		Texture->MipGenSettings = TMGS_NoMipmaps;
#endif
		Texture->Brightness = TextureData.Brightness;
		Texture->TextureMultiplier = TextureData.TextureMultiplier;

		// Update the texture with these new settings
		Texture->UpdateResource();

#if !WITH_EDITOR
		TextureData.Region = FUpdateTextureRegion2D(0, 0, 0, 0, TextureData.Width, TextureData.Height);

		Texture->UpdateTextureRegions(0, 1, &TextureData.Region, TextureData.Pitch, TextureData.BytesPerPixel, TextureData.ImageData, DataCleanupFunc );
#endif

		return Texture;
	}

	EActionResult::Type FSceneImporter::CreateTexture(FSceneGraphId ElementId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::CreateTexture);

		FAssetData& AssetData = AssetDataList[ElementId];
		FTextureData& TextureData = TextureDataList[ElementId];

		IDatasmithTextureElement* TextureElement = static_cast<IDatasmithTextureElement*>(Elements[ AssetData.ElementId ].Get());

		// If the load of the image has failed, cleanup the TextureData and return
		if (TextureData.Width == 0 || TextureData.Height == 0 || TextureData.ImageData == nullptr)
		{
			if (UObject* THelper = AssetData.GetObject<>())
			{
				FAssetRegistry::UnregisteredAssetsData(THelper, SceneKey, [](FAssetData& AssetData) -> void
					{
						AssetData.AddState(EAssetState::Completed);
						AssetData.Object.Reset();
					});
			}

			UE_LOGF(LogDatasmithRuntime, Warning, "Failed to create texture %ls", TextureElement->GetName());

			return EActionResult::Failed;
		}

		FDataCleanupFunc DataCleanupFunc;
		DataCleanupFunc = [this, ElementId](uint8* SrcData, const FUpdateTextureRegion2D* Regions) -> void
		{
			FTextureData& TextureData = this->TextureDataList[ElementId];

			FMemory::Free(TextureData.ImageData);
			TextureData.ImageData = nullptr;
		};

		UObject* THelper = FAssetRegistry::FindObjectFromHash(AssetData.Hash);
		ensure(THelper);

		UTexture* Texture = nullptr;

		if (TextureElement->GetTextureMode() == EDatasmithTextureMode::Ies)
		{
			Texture = CreateIESTexture(AssetData.GetObject<UTextureLightProfile>(), TextureData, TextureElement, DataCleanupFunc);
		}
		else
		{
			Texture = CreateImageTexture(AssetData.GetObject<UTexture2D>(), TextureData, TextureElement, DataCleanupFunc);
		}

		if (Texture)
		{
			// Apply metadata on newly created texture if any
			ApplyMetadata(AssetData.MetadataId, Texture);

			uint32 TextureHash = GetTypeHash(TextureElement->CalculateElementHash(true), EDataType::Texture);

			FAssetRegistry::UnregisteredAssetsData(THelper, SceneKey, [this, &Texture, TextureHash](FAssetData& AssetData) -> void
				{
					AssetData.Object = Texture;
					AssetData.Hash = TextureHash;
					FAssetRegistry::RegisterAssetData(Texture, this->SceneKey, AssetData);
				});

			FAssetRegistry::SetObjectCompletion(Texture, true);
		}
		else
		{
			FAssetRegistry::UnregisteredAssetsData(THelper, SceneKey, [this](FAssetData& AssetData) -> void
				{
					AssetData.AddState(EAssetState::Completed);
					AssetData.Object.Reset();
				});

			UE_LOGF(LogDatasmithRuntime, Warning, "Failed to create texture %ls", TextureElement->GetName());
		}

		ActionCounter.Increment();

		return Texture ? EActionResult::Succeeded : EActionResult::Failed;
	}

	bool FSceneImporter::LoadTexture(FSceneGraphId ElementId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::LoadTexture);

		FTextureData& TextureData = TextureDataList[ElementId];

		IDatasmithTextureElement* TextureElement = static_cast<IDatasmithTextureElement*>(Elements[ ElementId ].Get());

		// If image file does not exist, add scene's resource path if valid
		if (!FPaths::FileExists(TextureElement->GetFile()) && FPaths::DirectoryExists(SceneElement->GetResourcePath()))
		{
			TextureElement->SetFile( *FPaths::Combine(SceneElement->GetResourcePath(), TextureElement->GetFile()) );
		}

		bool bSuccessfulLoad = false;
		FString TextureName(TextureElement->GetName());

		if (TextureElement->GetTextureMode() == EDatasmithTextureMode::Ies)
		{
			bSuccessfulLoad = GetTextureDataForIes(TextureElement->GetFile(), TextureData);
		}
		else
		{
			const bool bCreateNormal = ( TextureElement->GetTextureMode() == EDatasmithTextureMode::Bump );
			bSuccessfulLoad = GetTextureDataFromFile(TextureElement->GetFile(), EDSResizeTextureMode::NearestPowerOfTwo, GMaxTextureDimensions, bCreateNormal, TextureData);

			if (!bSuccessfulLoad)
			{
				EDatasmithTextureFormat TextureFormat;
				uint32                  ByteCount;

				const uint8* Bytes = TextureElement->GetData(ByteCount, TextureFormat);

				if (Bytes != nullptr && ByteCount > 0)
				{
					TArray<uint8> ByteArray;
					ByteArray.SetNumUninitialized(ByteCount);
					FPlatformMemory::Memcpy(ByteArray.GetData(), Bytes, ByteCount);

					bSuccessfulLoad = GetTextureDataFromBuffer(ByteArray, TextureFormat, EDSResizeTextureMode::NearestPowerOfTwo, GMaxTextureDimensions, bCreateNormal, TextureData);
				}

			}
		}

		if (!bSuccessfulLoad)
		{
			if (TextureData.ImageData)
			{
				FMemory::Free(TextureData.ImageData);
			}

			TextureData.Width = 0;
			TextureData.Height = 0;
			TextureData.ImageData = nullptr;

			UE_LOGF(LogDatasmithRuntime, Warning, "Cannot load image file %ls for texture %ls", TextureElement->GetFile(), TextureElement->GetLabel());
		}
		else
		{
			TasksToComplete |= EWorkerTask::TextureAssign;
		}

		FActionTaskFunction CreateTaskFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
		{
			return this->CreateTexture(Referencer.GetId());
		};

		AddToQueue(EQueueTask::NonAsyncQueue, { CreateTaskFunc, {EDataType::Texture, ElementId, 0 } });

		ActionCounter.Increment();

		return true;
	}

	void FSceneImporter::ProcessTextureData(FSceneGraphId TextureId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneImporter::ProcessTextureData);

		FAssetData& AssetData = AssetDataList[TextureId];

		// Something is wrong. Do not go any further
		if (AssetData.HasState(EAssetState::PendingDelete))
		{
			UE_LOGF(LogDatasmithRuntime, Error, "A texture marked for deletion is actually used by the scene");
			return;
		}

		if (AssetData.HasState(EAssetState::Processed))
		{
			return;
		}

		IDatasmithTextureElement* TextureElement = static_cast<IDatasmithTextureElement*>(Elements[ TextureId ].Get());

		uint32 TextureHash = GetTypeHash(TextureElement->CalculateElementHash(true), EDataType::Texture);

		if (UObject* Asset = FAssetRegistry::FindObjectFromHash(TextureHash))
		{
			AssetData.SetState(EAssetState::Processed);

			AssetData.Hash = TextureHash;
			AssetData.Object = TWeakObjectPtr<UObject>(Asset);
			FAssetRegistry::RegisterAssetData(Asset, SceneKey, AssetData);

			return;
		}

		// The final texture has not been created yet, track texture data with temporary hash
		AssetData.Hash = HashCombine(SceneKey, TextureHash);

		if (UObject* Asset = FAssetRegistry::FindObjectFromHash(AssetData.Hash))
		{
			AssetData.SetState(EAssetState::Processed);

			AssetData.Object = TWeakObjectPtr<UObject>(Asset);

			FAssetRegistry::RegisterAssetData(Asset, SceneKey, AssetData);

			return;
		}

		FActionTaskFunction LoadTaskFunc = [this](UObject* Object, const FReferencer& Referencer) -> EActionResult::Type
		{
			OnGoingTasks.Emplace( Async(
#if WITH_EDITOR
				EAsyncExecution::LargeThreadPool,
#else
				EAsyncExecution::ThreadPool,
#endif
				[this, ElementId = Referencer.GetId()]()->bool
				{
					return this->LoadTexture(ElementId);
				},
				[this]()->void
				{
					this->ActionCounter.Increment();
				}
			));

			return EActionResult::Succeeded;
		};

		// Textures are added in two steps. Make sure the associated FTextureData is created
		if (!TextureDataList.Contains(TextureId))
		{
			TextureDataList.Add(TextureId);
		}

		AddToQueue(EQueueTask::TextureQueue, { LoadTaskFunc, {EDataType::Texture, TextureId, 0 } });
		TasksToComplete |= EWorkerTask::TextureLoad;

		// Create texture helper to leverage registration mechanism
		UDatasmithRuntimeTHelper* TextureHelper = NewObject< UDatasmithRuntimeTHelper >();

		AssetData.Object = TWeakObjectPtr<UObject>(TextureHelper);

		AssetData.SetState(EAssetState::Processed);

		FAssetRegistry::RegisterAssetData(TextureHelper, SceneKey, AssetData);

		TextureElementSet.Add(TextureId);
	}
} // End of namespace DatasmithRuntime

