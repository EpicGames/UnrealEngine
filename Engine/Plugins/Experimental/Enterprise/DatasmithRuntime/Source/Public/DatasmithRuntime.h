// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DatasmithImportOptions.h"
#include "DatasmithTranslatableSource.h"
#include "DirectLink/DatasmithSceneReceiver.h"

#include "Async/Future.h"
#include "Containers/Queue.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "BodySetupEnums.h"

#include <atomic>

#include "DatasmithRuntime.generated.h"

class ADatasmithRuntimeActor;
class FDatasmithReferenceMaterialSelector;
class FEvent;
class IDatasmithScene;
class IDatasmithTranslator;
class UDatasmithCommonTessellationOptions;
class UDatasmithOptionsBase;

namespace DatasmithRuntime
{
	class FSceneImporter;
	class FDestinationProxy;
}

struct FUpdateContext
{
	TArray<TSharedPtr<IDatasmithElement>> Additions;
	TArray<TSharedPtr<IDatasmithElement>> Updates;
	TArray<DirectLink::FSceneGraphId> Deletions;
};

namespace DatasmithRuntime
{
	class FTranslationJob
	{
	public:
		FTranslationJob(ADatasmithRuntimeActor* InActor, const FString& InFilePath)
			: RuntimeActor(InActor)
			, FilePath(InFilePath)
		{
		}

		FTranslationJob()
		{
		}

		bool Execute();

	private:
		TWeakObjectPtr<ADatasmithRuntimeActor> RuntimeActor;
		FString FilePath;
	};

	class FTranslationThread
	{
	public:
		FTranslationThread() 
			: bKeepRunning(false)
		{}

		~FTranslationThread();

		void Run();

		void AddJob(FTranslationJob&& Job)
		{
			JobQueue.Enqueue(MoveTemp(Job));
		}

		std::atomic_bool bKeepRunning;
		TFuture<void> ThreadResult;
		FEvent* ThreadEvent = nullptr;
		TQueue< FTranslationJob, EQueueMode::Mpsc > JobQueue;
	};
}

// UHT doesn't really like operator ::
using FDatasmithSceneReceiver_ISceneChangeListener = FDatasmithSceneReceiver::ISceneChangeListener;

UENUM(BlueprintType)
enum class EBuildHierarchyMethod : uint8
{
	None,
	Simplified,
	Unfiltered,
};

USTRUCT(BlueprintType)
struct FDatasmithRuntimeImportOptions
{
	GENERATED_BODY()

	/** Tessellation options for CAD import */
	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadWrite)
	FDatasmithTessellationOptions TessellationOptions;

	/**
	 * Indicates whether a hierarchy of actors should be built or not.
	 * In the case a hierarchy is built, it can be simplified to minimize the number of actors created
	 * By default, a simplified hierarchy is built
	 */
	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadWrite)
	EBuildHierarchyMethod BuildHierarchy = EBuildHierarchyMethod::Simplified;

	/**
	 * Indicates the type of collision for components
	 * Set to ECollisionEnabled::QueryOnly (spatial queries, no physics) by default
	 */
	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadWrite)
	TEnumAsByte<ECollisionEnabled::Type> BuildCollisions = ECollisionEnabled::QueryOnly;

	/**
	 * Indicates the type of collision for static meshes
	 * Set to ECollisionTraceFlag::CTF_UseComplexAsSimple by default
	 */
	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadWrite)
	TEnumAsByte<ECollisionTraceFlag> CollisionType = ECollisionTraceFlag::CTF_UseComplexAsSimple;
	/**
	 * Indicates whether meta-data should be imported or not
	 * Meta-data are imported by default
	 */
	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadWrite)
	bool bImportMetaData = true;
};

UCLASS(meta = (DisplayName = "Datasmith Destination"))
class DATASMITHRUNTIME_API ADatasmithRuntimeActor
	: public AActor
	, public FDatasmithSceneReceiver_ISceneChangeListener
{
	GENERATED_BODY()

public:
	ADatasmithRuntimeActor();

	// AActor overrides
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void Tick(float DeltaSeconds) override;
	// End AActor overrides

	// ISceneChangeListener interface
	virtual void OnOpenDelta(/*int32 ElementsCount*/) override;
	virtual void OnNewScene(const DirectLink::FSceneIdentifier& SceneId) override;
	virtual void OnAddElement(DirectLink::FSceneGraphId ElementId, TSharedPtr<IDatasmithElement> Element) override;
	virtual void OnChangedElement(DirectLink::FSceneGraphId ElementId, TSharedPtr<IDatasmithElement> Element) override;
	virtual void OnRemovedElement(DirectLink::FSceneGraphId ElementId) override;
	virtual void OnCloseDelta() override;
	// End ISceneChangeListener interface

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	bool IsConnected();

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	FString GetDestinationName() { return GetName(); }

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	FString GetSourceName();

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	bool OpenConnectionWithIndex(int32 SourceIndex);

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	void CloseConnection();

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	int32 GetSourceIndex();

	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadOnly)
	float Progress;

	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadOnly)
	bool bBuilding;

	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadOnly)
	FString LoadedScene;

	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadWrite)
	FDatasmithRuntimeImportOptions ImportOptions;

	UPROPERTY(Category = "DatasmithRuntime", EditDefaultsOnly, BlueprintReadOnly)
	FString ExternalFile;

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	bool IsReceiving() { return bReceivingStarted; }

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	void Reset();

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	bool LoadFile(const FString& FilePath);

	void SetScene(TSharedPtr<IDatasmithScene> InSceneElement) { SceneElement = InSceneElement; }

	void ApplyNewScene();

	virtual void OnImportEnd();

	static void OnShutdownModule();
	static void OnStartupModule();

private:
	TSharedPtr< DatasmithRuntime::FSceneImporter > SceneImporter;

	TSharedPtr<DatasmithRuntime::FDestinationProxy> DirectLinkHelper;

	TSharedPtr<IDatasmithScene>      SceneElement;
	TSharedPtr<IDatasmithTranslator> Translator;

	std::atomic_bool bNewScene;
	std::atomic_bool bReceivingStarted;
	std::atomic_bool bReceivingEnded;

	float ElementDeltaStep;

	static std::atomic_bool bImportingScene;
	FUpdateContext UpdateContext;

	static TUniquePtr<DatasmithRuntime::FTranslationThread> TranslationThread;

	friend class DatasmithRuntime::FTranslationJob;
};
