// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/StrongObjectPtr.h"

#include "DatasmithRuntimeBlueprintLibrary.generated.h"

class ADatasmithRuntimeActor;
class UWorld;

namespace DatasmithRuntime
{
	class FDirectLinkProxyImpl;
}

USTRUCT(BlueprintType)
struct DATASMITHRUNTIME_API FDatasmithRuntimeSourceInfo
{
	GENERATED_USTRUCT_BODY()

	FDatasmithRuntimeSourceInfo()
	{
	}

	FDatasmithRuntimeSourceInfo( const FString& InName, const FGuid& InHandle)
		: Name(InName)
		, SourceHandle(InHandle)
	{
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DatasmithRuntime")
	FString Name;

	FGuid SourceHandle;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FDatasmithRuntimeChangeEvent);

// Class to interface with the DirectLink end point
UCLASS(BlueprintType)
class DATASMITHRUNTIME_API UDirectLinkProxy : public UObject
{
	GENERATED_BODY()

public:
	UDirectLinkProxy();
	~UDirectLinkProxy();

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	FString GetEndPointName();

	UFUNCTION(BlueprintCallable, Category = "DirectLink")
	TArray<FDatasmithRuntimeSourceInfo> GetListOfSources();

	// Dynamic delegate used to trigger an event in the Game when there is
	// a change in the DirectLink network
	UPROPERTY(BlueprintAssignable)
	FDatasmithRuntimeChangeEvent OnDirectLinkChange;

	friend class UDatasmithRuntimeLibrary;
};

UCLASS(meta = (ScriptName = "DatasmithRuntimeLibrary"))
class DATASMITHRUNTIME_API UDatasmithRuntimeLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Load a file using the Datasmith translator associated with it
	 * @param DatasmithRuntimeActor	The actor to load the file into
	 * @param FilePath The path to the file to load.
	 * @return	true if an associated translator has been 
	 */
	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	static bool LoadFile(ADatasmithRuntimeActor* DatasmithRuntimeActor, const FString& FilePath);

	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	static void ResetActor(ADatasmithRuntimeActor* DatasmithRuntimeActor);

	/** Returns an interface to the DirectLink end point */
	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntime")
	static UDirectLinkProxy* GetDirectLinkProxy();

	/**
	 * Open a file browser to select a file and call LoadFile with the selected file
	 * @param DatasmithRuntimeActor	The actor to load the file into
	 * @param DefaultPath Path to open the file browser in.
	 * @return	true if an associated translator has been 
	 */
	UFUNCTION(BlueprintCallable, Category = "DatasmithRuntimeHelper")
	static bool LoadFileFromExplorer(ADatasmithRuntimeActor* DatasmithRuntimeActor, const FString& DefaultPath);
};
