// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Stats/Stats.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#define DATASMITHLIVEUPDATE_MODULE_NAME TEXT("DatasmithRuntime")

class IDatasmithRuntimeModuleInterface : public IModuleInterface
{
public:
	static IDatasmithRuntimeModuleInterface& Get()
	{
		return FModuleManager::LoadModuleChecked< IDatasmithRuntimeModuleInterface >(DATASMITHLIVEUPDATE_MODULE_NAME);
	}

	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DATASMITHLIVEUPDATE_MODULE_NAME);
	}
};